{
	"FileVersion": 1,

	"FriendlyName": "MutablePopulation",
	"Version": 1,
	"VersionName": "1.8.0",
	"CreatedBy": "Epic Games, Inc.",
	"Description": "Extend the Mutable plugin to support Population assets.",
	"Category": "CustomizableObjects",
	"Modules": [
		{
			"Name": "CustomizableObjectPopulation",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "CustomizableObjectPopulationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Mutable",
			"Enabled": true
		}
	],

	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"IsBetaVersion": false,
	"EnabledByDefault": false
}
