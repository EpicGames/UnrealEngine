// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CustomizableObjectPopulation : ModuleRules
{
	public CustomizableObjectPopulation(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		ShortName = "MuCOP";

		DefaultBuildSettings = BuildSettingsVersion.Latest;

        PrivateDependencyModuleNames.AddRange(new string[] {
            "InputCore",
            "SlateCore",
            "CoreUObject",
            "RenderCore",
            "RHI",
            "AppFramework",
            "Projects",
            "ApplicationCore",
			"ClothingSystemRuntimeCommon",
			"ClothingSystemRuntimeInterface",
        });

        PublicDependencyModuleNames.AddRange(new string[] {
            "Slate",
            "Core",
            "Engine",
			"CustomizableObject",
        });

        PrivateIncludePathModuleNames.AddRange(
        new string[] {
		});

        if (TargetRules.bBuildEditor == true)
        {
            PublicDependencyModuleNames.Add("UnrealEd");	// @todo api: Only public because of WITH_EDITOR
            PublicDependencyModuleNames.Add("DerivedDataCache");
            PublicDependencyModuleNames.Add("EditorStyle");
            PublicDependencyModuleNames.Add("MessageLog");
			PrivateIncludePathModuleNames.Add("CustomizableObjectEditor");
		}
    }
}
