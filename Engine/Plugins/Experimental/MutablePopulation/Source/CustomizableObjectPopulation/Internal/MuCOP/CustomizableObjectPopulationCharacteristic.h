// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CustomizableObjectPopulationCharacteristic.generated.h"

struct FCustomizableObjectPopulationConstraint;

USTRUCT()
struct FCustomizableObjectPopulationCharacteristic
{
public:

	GENERATED_USTRUCT_BODY()
	
	/** Name of the parameter to add contraints to */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere, meta = (ToolTip = "Name of the Customizable Object Parameter that will be specified by this characteristic"))
	FString ParameterName;
	
	/** List of constraints */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere, meta = (ToolTip = "Constraints applied to this characteristic"))
	TArray<FCustomizableObjectPopulationConstraint> Constraints;

};
