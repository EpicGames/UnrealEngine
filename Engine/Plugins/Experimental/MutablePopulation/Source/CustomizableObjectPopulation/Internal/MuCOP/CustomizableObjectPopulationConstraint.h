// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "CustomizableObjectPopulationConstraint.generated.h"

class UCurveBase;

UENUM()
enum class EPopulationConstraintType : uint8
{
	NONE,
	BOOL,
	DISCRETE,
	DISCRETE_FLOAT,
	DISCRETE_COLOR,
	TAG,
	RANGE,
	CURVE,
	CURVE_COLOR
};


UENUM()
enum class ECurveColor : uint8
{
	RED = 0,
	GREEN = 1,
	BLUE = 2,
	ALPHA = 3
};


USTRUCT()
struct FConstraintRanges
{
public:

	GENERATED_USTRUCT_BODY()

	FConstraintRanges():
		MinimumValue(0.0f),
		MaximumValue(0.0f),
		RangeWeight(1)
	{}
	
	/** Minimum value of the range */
	UPROPERTY(Category = "RangeConstraint", EditAnywhere, meta = (ToolTip = "Minimum value defining this range. Can't be higher than the maximum"))
	float MinimumValue;

	/** Maximum value of the range */
	UPROPERTY(Category = "RangeConstraint", EditAnywhere, meta = (ToolTip = "Maximum value defining this range. Can't be lower than the minimum"))
	float MaximumValue;

	/** integer used to decide which range apply when multiple ranges are used for the same characteristic */
	UPROPERTY(Category = "RangeConstraint", EditAnywhere, meta = (ToolTip = "How often this range will be used among the ranges defined by this constraint"))
	int32 RangeWeight;

};


USTRUCT()
struct FCustomizableObjectPopulationConstraint
{
public:

	GENERATED_USTRUCT_BODY()

	FCustomizableObjectPopulationConstraint() : 
		Type(EPopulationConstraintType::NONE),
		ConstraintWeight(1),
		TrueWeight(1),
		FalseWeight(1),
		DiscreteValue(""),
		DiscreteColor(FLinearColor(EForceInit::ForceInit)),
		Curve(nullptr),
		CurveColor(ECurveColor::RED)
	{};

public:

	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	EPopulationConstraintType Type;
	/** Integer used to decide which constraint to apply when multiple constraints are used for the same characteristic */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	int32 ConstraintWeight;

	/** Bool Constraint (Zero to only one of them if we want to force one option.Can't contain two zeroes) */
	UPROPERTY(Category = "BoolConstraint", EditAnywhere)
	int32 TrueWeight;

	UPROPERTY(Category = "BoolConstraint", EditAnywhere)
	int32 FalseWeight;

	/** Discrete Constraint */
	/** Name of the int parameter option chosen */
	UPROPERTY(Category = "DiscreteConstraint", EditAnywhere)
	FString DiscreteValue;

	/** Color chosen **/
	UPROPERTY(Category = "DiscreteConstraint", EditAnywhere)
	FLinearColor DiscreteColor;

	/** Tag Constraint */
	/** List of tags that force a parameter */
	UPROPERTY(Category = "TagConstraint", EditAnywhere)
	TArray<FString> Allowlist;

	/** List of tags that exclude a parameter */
	UPROPERTY(Category = "TagConstraint", EditAnywhere)
	TArray<FString> Blocklist;

	/** Ranges Constraint */
	UPROPERTY(Category = "RangeConstraint", EditAnywhere)
	TArray<FConstraintRanges> Ranges;

	/** Curve Constraint */
	UPROPERTY(Category = "CurveConstraint", EditAnywhere)
	TObjectPtr<UCurveBase> Curve;

	UPROPERTY(Category = "CurveConstraint", EditAnywhere)
	ECurveColor CurveColor;
};


//USTRUCT()
//struct FCustomizableObjectPopulationBoolConstraint : public FCustomizableObjectPopulationConstraint
//{
//public:
//
//	GENERATED_USTRUCT_BODY()
//
//public:
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	int32 TrueWeight;
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	int32 FalseWeight;
//
//};
//
//USTRUCT()
//struct FCustomizableObjectPopulationDiscreteConstraint : public FCustomizableObjectPopulationConstraint
//{
//public:
//
//	GENERATED_USTRUCT_BODY()
//
//public:
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	int32 DiscreteWeight;
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	int32 Value;
//
//};
//
//USTRUCT()
//struct FCustomizableObjectPopulationTagConstraint : public FCustomizableObjectPopulationConstraint
//{
//public:
//
//	GENERATED_USTRUCT_BODY()
//
//public:
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	FString Allowlist;
//	
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	FString Blocklist;
//
//};
//
//
//
//USTRUCT()
//struct FCustomizableObjectPopulationRangeConstraint : public FCustomizableObjectPopulationConstraint
//{
//public:
//
//	GENERATED_USTRUCT_BODY()
//
//public:
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	int32 RangeWeight;
//
//	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
//	TArray<FConstraintRanges> Ranges;
//
//};
//
//
//USTRUCT()
//struct FCustomizableObjectPopulationCurveConstraint : public FCustomizableObjectPopulationConstraint
//{
//public:
//
//	GENERATED_USTRUCT_BODY()
//
//public:
//
//	//TODO(Anticto): Search for the unreal asset that allows to draw a curve
//
//};
//
