// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOP/CustomizableObjectPopulation.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOP/CustomizableObjectPopulationCharacteristic.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"
#include "MuCOP/CustomizableObjectPopulationConstraint.h"
#include "MuCOP/CustomizableObjectPopulationCustomVersion.h"
#include "MuCOP/CustomizableObjectPopulationGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectPopulation)

class ITargetPlatform;
class UCustomizableObjectInstance;

#if WITH_EDITOR
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "MuCOP/CustomizableObjectPopulationGeneratorPrivate.h"
#endif

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulation"

using namespace CustomizableObjectPopulation;

UCustomizableObjectPopulation::UCustomizableObjectPopulation()
{
	Generator = NewObject<UCustomizableObjectPopulationGenerator>();
}

int32 UCustomizableObjectPopulation::GeneratePopulation(TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate) const
{
	if (Generator)
	{
		return Generator->GeneratePopulation(OutInstances, NumInstancesToGenerate);
	}

	return -1;
}

bool UCustomizableObjectPopulation::RegeneratePopulation(int32 Seed, TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate) const
{
	if (Generator)
	{
		Generator->RegeneratePopulation(Seed, OutInstances, NumInstancesToGenerate);
		return true;
	}

	return false;
}

bool UCustomizableObjectPopulation::IsValidPopulation() const
{
	for (int32 i = 0; i < ClassWeights.Num(); ++i)
	{
		if (ClassWeights[i].Class == nullptr)
		{
			return false;
		}
	}

	return true;
}

bool UCustomizableObjectPopulation::HasGenerator() const
{
	return Generator!=nullptr;
}


#if WITH_EDITOR

FCustomizableObjectPopulationClassGenerator CompilePopulationClass(UCustomizableObjectPopulationClass* InPopulationClass);

void UCustomizableObjectPopulation::CompilePopulation(UCustomizableObjectPopulationGenerator* NewGenerator)
{
	if (NewGenerator)
	{
		Generator = NewGenerator;
	}

	CompilePopulationInternal(this);
}


void UCustomizableObjectPopulation::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	if (IsValidPopulation())
	{
		for (int32 i = 0; i < ClassWeights.Num(); ++i)
		{
			check(ClassWeights[i].Class->CustomizableObject->IsCompiled());
		}

		CompilePopulation(Generator);
	}
}

bool UCustomizableObjectPopulation::IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform)
{
	return Generator && Generator->IsCompiledForCook();
}

void UCustomizableObjectPopulation::CompilePopulationInternal(UCustomizableObjectPopulation* InPopulation)
{
	if (!Generator)
	{
		return;
	}	 

	TArray<FClassWeightPair> SortedClassWeights = InPopulation->ClassWeights;

	// Sort by name so initial order does not affect generation order.
	SortedClassWeights.Sort(
		[](const FClassWeightPair& A, const FClassWeightPair& B)
	{ return A.Class->Name.Compare(B.Class->Name) < 0; });

	TArray<UCustomizableObject*> PopulationObjects;
	TSharedPtr<FCustomizableObjectPopulationGeneratorPrivate> Private = MakeShareable(new FCustomizableObjectPopulationGeneratorPrivate());

	PopulationObjects.SetNum(SortedClassWeights.Num());
	for (int32 I = 0; I < SortedClassWeights.Num(); ++I)
	{
		PopulationObjects[I] = SortedClassWeights[I].Class->CustomizableObject;
	}

	TArray<int32> SamplerWeights;
	SamplerWeights.SetNum(SortedClassWeights.Num());
	for (int32 I = 0; I < SamplerWeights.Num(); ++I)
	{
		SamplerWeights[I] = SortedClassWeights[I].ClassWeight;
	}

	Private->ClassSampler = FPopulationClassSampler(SamplerWeights);

	const int32 NumClasses = SortedClassWeights.Num();
	Private->ClassGenerators.Empty(NumClasses);
	for (int32 I = 0; I < NumClasses; ++I)
	{
		Private->ClassGenerators.Emplace(CompilePopulationClass(SortedClassWeights[I].Class));
	}

	Generator->Init(PopulationObjects, Private);
}


namespace
{
	bool IsAnyTagInList(const TArray<FString>& Tags, const TArray<FString>& List)
	{
		for (const FString& Tag : Tags)
		{
			if (List.Contains(Tag))
			{
				return true;
			}
		}

		return false;
	}

} //namespce


void CompileBoolSamplers(
	TArray<FString>& InBoolParamNames,
	UCustomizableObjectPopulationClass* PopulationClass,
	FCustomizableObjectPopulationClassGenerator& InOutGenerator)
{
	// Sort parameters names so the same parameters generate in the same order regardless of 
	// initial order (this is only valid if we have the same parameters).
	InBoolParamNames.Sort([](const FString& A, const FString& B) { return A.Compare(B) < 0; });

	for (FString& BoolParamName : InBoolParamNames)
	{
		FParameterTags* ParamTagsFound = PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Find(BoolParamName);

		const TArray<FString> EmptyTagArray;
		const TArray<FString>& ParamTags = ParamTagsFound ? ParamTagsFound->Tags : EmptyTagArray;

		const bool bIsAllowlisted = IsAnyTagInList(ParamTags, PopulationClass->Allowlist);
		const bool bIsBlocklisted = IsAnyTagInList(ParamTags, PopulationClass->Blocklist);
		const bool bIsTagged = bIsAllowlisted || bIsBlocklisted;

		// If not tagged False and True weights will be set to 1.
		int32 FalseParamWeight = static_cast<int32>(bIsBlocklisted || !bIsTagged);
		int32 TrueParamWeight = static_cast<int32>((bIsAllowlisted && !bIsBlocklisted) || !bIsTagged);

		FCustomizableObjectPopulationCharacteristic* CharacteristicFound = PopulationClass->Characteristics.FindByPredicate(
			[&BoolParamName](FCustomizableObjectPopulationCharacteristic& Char) { return Char.ParameterName == BoolParamName; });

		int32 ValidConstraintsFound = 0;
		int32 InvalidConstraintsFound = 0;

		TArray<int32> ConstraintWeights;
		TArray<FConstraintIndex> SamplerIDs;

		if (CharacteristicFound)
		{
			for (FCustomizableObjectPopulationConstraint& C : CharacteristicFound->Constraints)
			{
				const bool bIsValidType = C.Type == EPopulationConstraintType::BOOL ||
					C.Type == EPopulationConstraintType::TAG;

				if (!bIsValidType)
				{
					++InvalidConstraintsFound;
					continue;
				}

				++ValidConstraintsFound;

				if (ValidConstraintsFound > 1)
				{
					continue;
				}

				// Take the first valid constraint. only one allowed for bools. (should always be the first one)
				if (C.Type == EPopulationConstraintType::BOOL)
				{
					TrueParamWeight = FMath::Max(C.TrueWeight, 0);
					FalseParamWeight = FMath::Max(C.FalseWeight, 0);

					if (C.TrueWeight + C.FalseWeight <= 0)
					{
						--ValidConstraintsFound;
					}
					else
					{
						InOutGenerator.BoolSamplers.Emplace(TrueParamWeight, FalseParamWeight);
						SamplerIDs.Add(FConstraintIndex(InOutGenerator.BoolSamplers.Num() - 1, EPopulationSamplerType::BOOL));
						ConstraintWeights.Add(C.ConstraintWeight);
					}
				}
				else if (C.Type == EPopulationConstraintType::TAG)
				{
					const bool bConstraintBlocklisted = IsAnyTagInList(ParamTags, C.Blocklist);
					const bool bConstraintAllowlisted = IsAnyTagInList(ParamTags, C.Allowlist);
					const bool bIsConstraintTagged = bConstraintBlocklisted || bConstraintAllowlisted;

					// Simple override of parameter tag listing, if any. Tag Constraint Weight is omited.
					if (bIsConstraintTagged)
					{
						TrueParamWeight = static_cast<int32>(bConstraintAllowlisted && !bConstraintBlocklisted);
						FalseParamWeight = static_cast<int32>(bConstraintBlocklisted);

						InOutGenerator.BoolSamplers.Emplace(TrueParamWeight, FalseParamWeight);
						SamplerIDs.Add(FConstraintIndex(InOutGenerator.BoolSamplers.Num() - 1, EPopulationSamplerType::BOOL));
						ConstraintWeights.Add(C.ConstraintWeight);
					}
				}
			}
		}
 
		if (ValidConstraintsFound < 1)
		{
			InOutGenerator.BoolSamplers.Emplace(1, 1);
			SamplerIDs.Add(FConstraintIndex(InOutGenerator.BoolSamplers.Num() - 1, EPopulationSamplerType::BOOL));
			ConstraintWeights.Add(1);
		}

		InOutGenerator.ConstraintSamplers.Emplace(ConstraintWeights, BoolParamName, SamplerIDs);
	}
}


void CompileFloatSamplers(
	TArray<FString>& InFloatParamNames,
	UCustomizableObjectPopulationClass* PopulationClass,
	FCustomizableObjectPopulationClassGenerator& InOutGenerator)
{
	// Sort parameters names so the same parameters generate in the same order regardless of 
	// initial order (this is only valid if we have the same parameters).
	InFloatParamNames.Sort([](const FString& A, const FString& B) { return A.Compare(B) < 0; });

	for (FString& FloatParamName : InFloatParamNames)
	{
		TArray<int32> ConstraintWeights;
		TArray<FConstraintIndex> SamplerIDs;
		FCustomizableObjectPopulationCharacteristic* CharacteristicFound = PopulationClass->Characteristics.FindByPredicate(
			[&FloatParamName](FCustomizableObjectPopulationCharacteristic& C) { return C.ParameterName == FloatParamName; });

		if (!CharacteristicFound)
		{
			InOutGenerator.UniformFloatSamplers.Emplace(FFloatUniformSampler());
			SamplerIDs.Add(FConstraintIndex(InOutGenerator.UniformFloatSamplers.Num() - 1, EPopulationSamplerType::UNIFORM_FLOAT));
			ConstraintWeights.Add(1);
			InOutGenerator.ConstraintSamplers.Emplace(ConstraintWeights, FloatParamName, SamplerIDs);
			continue;
		}

		int32 ValidConstraintsFound = 0;
		int32 InvalidConstraintsFound = 0;

		TArray<FCustomizableObjectPopulationConstraint>& Constraints = CharacteristicFound->Constraints;
		for (int32 I = 0; I < Constraints.Num(); ++I)
		{
			FCustomizableObjectPopulationConstraint& C = Constraints[I];

			const bool bIsValidType = C.Type == EPopulationConstraintType::CURVE ||
				C.Type == EPopulationConstraintType::RANGE ||
				C.Type == EPopulationConstraintType::DISCRETE_FLOAT;

			if (!bIsValidType)
			{
				++InvalidConstraintsFound;
				continue;
			}

			++ValidConstraintsFound;

			if (C.Type == EPopulationConstraintType::CURVE)
			{
				if (C.Curve)
				{
					if (UCurveFloat* FloatCurve = Cast<UCurveFloat>(C.Curve))
					{
						InOutGenerator.CurveSamplers.Emplace(FloatCurve->FloatCurve);
					}
					else if (UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(C.Curve))
					{
						InOutGenerator.CurveSamplers.Emplace(ColorCurve->FloatCurves[(int32)C.CurveColor]);
					}
					else if (UCurveVector* VectorCurve = Cast<UCurveVector>(C.Curve))
					{
						InOutGenerator.CurveSamplers.Emplace(VectorCurve->FloatCurves[(int32)C.CurveColor]);
					}
					SamplerIDs.Add(FConstraintIndex(InOutGenerator.CurveSamplers.Num() - 1, EPopulationSamplerType::CURVE));
					ConstraintWeights.Add(C.ConstraintWeight);
				}
				else
				{
					--ValidConstraintsFound;
				}
			}
			else if (C.Type == EPopulationConstraintType::RANGE || C.Type == EPopulationConstraintType::DISCRETE_FLOAT)
			{
				if (C.Ranges.Num() == 0)
				{
					--ValidConstraintsFound;
				}
				else
				{
					TArray<TTuple<float, float>> SamplerRanges;
					TArray<int32> SamplerWeights;
					SamplerRanges.SetNum(C.Ranges.Num());
					SamplerWeights.SetNum(C.Ranges.Num());
					for (int32 RangeIdx = 0; RangeIdx < C.Ranges.Num(); ++RangeIdx)
					{
						const FConstraintRanges& Range = C.Ranges[RangeIdx];
						SamplerRanges[RangeIdx] = MakeTuple(Range.MinimumValue, Range.MaximumValue);
						SamplerWeights[RangeIdx] = Range.RangeWeight;
					}
					InOutGenerator.RangesSamplers.Emplace(SamplerWeights, SamplerRanges);
					SamplerIDs.Add(FConstraintIndex(InOutGenerator.RangesSamplers.Num() - 1, EPopulationSamplerType::RANGE));
					ConstraintWeights.Add(C.ConstraintWeight);
				}
			}
		}

		if (ValidConstraintsFound < 1)
		{
			InOutGenerator.UniformFloatSamplers.Emplace(FFloatUniformSampler());
			SamplerIDs.Add(FConstraintIndex(InOutGenerator.UniformFloatSamplers.Num() - 1, EPopulationSamplerType::UNIFORM_FLOAT));
			ConstraintWeights.Add(1);
		}

		InOutGenerator.ConstraintSamplers.Emplace(ConstraintWeights, FloatParamName, SamplerIDs);
	}
}


void CompileColorSamplers(
	TArray<FString>& InColorParamNames,
	UCustomizableObjectPopulationClass* PopulationClass,
	FCustomizableObjectPopulationClassGenerator& InOutGenerator)
{
	// Sort parameters names so the same parameters generate in the same order regardless of 
	// initial order (this is only valid if we have the same parameters).
	InColorParamNames.Sort([](const FString& A, const FString& B) { return A.Compare(B) < 0; });

	for (FString& ColorParamName : InColorParamNames)
	{
		TArray<int32> ConstraintWeights;
		TArray<FConstraintIndex> SamplerIDs;
		FCustomizableObjectPopulationCharacteristic* CharacteristicFound = PopulationClass->Characteristics.FindByPredicate(
			[&ColorParamName](FCustomizableObjectPopulationCharacteristic& C) { return C.ParameterName == ColorParamName; });

		if (!CharacteristicFound)
		{
			//// TODO: Uniform color sampler (with all colors? or standard hue rainbow with value and saturation 1?), alpha 1.0 constant.
			continue;
		}

		int32 ValidConstraintsFound = 0;
		int32 InvalidConstraintsFound = 0;

		TArray<FCustomizableObjectPopulationConstraint>& Constraints = CharacteristicFound->Constraints;
		for (int32 I = 0; I < Constraints.Num(); ++I)
		{
			FCustomizableObjectPopulationConstraint& C = Constraints[I];

			const bool bIsValidType = C.Type == EPopulationConstraintType::CURVE_COLOR
				|| C.Type == EPopulationConstraintType::DISCRETE_COLOR;

			if (!bIsValidType)
			{
				++InvalidConstraintsFound;
				continue;
			}


			if (C.Type == EPopulationConstraintType::CURVE_COLOR)
			{
				if (C.Curve)
				{
					if (UCurveLinearColor* ColorCurve = Cast<UCurveLinearColor>(C.Curve))
					{
						InOutGenerator.UniformColorSamplers.Emplace(*ColorCurve);
						SamplerIDs.Add(FConstraintIndex(InOutGenerator.UniformColorSamplers.Num() - 1, EPopulationSamplerType::UNIFORM_CURVE_COLOR));
						ConstraintWeights.Add(C.ConstraintWeight);
						++ValidConstraintsFound;
					}
				}
			}
			else if (C.Type == EPopulationConstraintType::DISCRETE_COLOR)
			{
				InOutGenerator.ConstantColors.Emplace(C.DiscreteColor);
				SamplerIDs.Add(FConstraintIndex(InOutGenerator.ConstantColors.Num() - 1, EPopulationSamplerType::CONSTANT_COLOR));
				ConstraintWeights.Add(C.ConstraintWeight);
				++ValidConstraintsFound;
			}
		}

		if (ValidConstraintsFound < 1)
		{
			//// TODO: Uniform color sampler (with all colors? or standard hue rainbow with value and saturation 1?), alpha 1.0 constant.
		}
		else
		{
			InOutGenerator.ConstraintSamplers.Emplace(ConstraintWeights, ColorParamName, SamplerIDs);
		}
	}
}


namespace
{
	void WeightParamOptionsTags(
		const TArray<FString>& Allowlist,
		const TArray<FString>& Blocklist,
		const TArray<TArray<FString>>& InOptionTags,
		TArray<int32>& InOutOptionWeights)
	{
		bool foundAnyTagRelevant = false;
		const int32 NumOptions = InOptionTags.Num();

		// Set all in Allowlist as one while we check if the Allowlist is valid
		bool bSomeOptionHasSomeTagInAllowlist = false;
		for (int32 I = 0; I < NumOptions; ++I)
		{
			if (IsAnyTagInList(InOptionTags[I], Allowlist))
			{
				InOutOptionWeights[I] = 1;
				bSomeOptionHasSomeTagInAllowlist = true;
			}
		}

		// Set all in Blocklist as zero, potentially overriding Allowlist sets while we check if the Blocklist is valid
		bool bSomeOptionHasSomeTagInBlocklist = false;
		for (int32 I = 0; I < NumOptions; ++I)
		{
			if (IsAnyTagInList(InOptionTags[I], Blocklist))
			{
				InOutOptionWeights[I] = 0;
				bSomeOptionHasSomeTagInBlocklist = true;
			}
			else if (bSomeOptionHasSomeTagInAllowlist && !IsAnyTagInList(InOptionTags[I], Allowlist))
			{
				InOutOptionWeights[I] = 0;
			}
		}

		// Reset all non black-listed to one because there is no one valid in the Allowlist
		if (!bSomeOptionHasSomeTagInAllowlist && bSomeOptionHasSomeTagInBlocklist) {
			for (int32 I = 0; I < NumOptions; ++I)
			{
				if (!IsAnyTagInList(InOptionTags[I], Blocklist))
				{
					InOutOptionWeights[I] = 1;
				}
			}
		}
	}

	void WeightParamCharacteristicDiscreteOptions(
		const FCustomizableObjectPopulationConstraint& Constraint,
		TArray<int32>& InOutDiscreteWeights,
		TArray<FString> ParameterOptions)
	{
		check(InOutDiscreteWeights.Num() == ParameterOptions.Num());
		//// This implementation is only valid while each characteristic is limited to one constraint and discrete weights can not be set to anything else than one
		int32 OptionIndex = ParameterOptions.Find(Constraint.DiscreteValue);

		for (int32 I = 0; I < InOutDiscreteWeights.Num(); ++I)
		{
			InOutDiscreteWeights[I] = (I == OptionIndex ? FMath::Max(Constraint.ConstraintWeight, 0) : 0);
		}

		//// This implementation is only discarded while each characteristic is limited to one constraint and discrete weights can not be set to anything else than one
		//int32 OptionIndex = ParameterOptions.Find(Constraint.DiscreteValue);

		//if (OptionIndex != INDEX_NONE)
		//{
		//	InOutDiscreteWeights[OptionIndex] = FMath::Max(Constraint.ConstraintWeight, 0);
		//}
	}

} //namespace


void CompileIntSamplers(
	TArray<FString>& InIntParamNames,
	UCustomizableObjectPopulationClass* PopulationClass,
	FCustomizableObjectPopulationClassGenerator& InOutGenerator)
{
	InIntParamNames.Sort([](const FString& A, const FString& B) { return A.Compare(B) < 0; });

	//IntSamplers.Empty(InIntParamNames.Num());
	for (FString& IntParamName : InIntParamNames)
	{
		FParameterTags* ParamTagsFound = PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Find(IntParamName);

		TArray<FString> ParamTags = ParamTagsFound ? ParamTagsFound->Tags : TArray<FString>();

		// If the parameter is block listed
		const bool bIsBlocklisted = IsAnyTagInList(ParamTags, PopulationClass->Blocklist);

		const int32 ParamIndex = PopulationClass->CustomizableObject->GetPrivate()->FindParameter(IntParamName);
		const int32 NumOptions = PopulationClass->CustomizableObject->GetPrivate()->GetEnumParameterNumValues(ParamIndex);

		TArray<FString> Options;
		TArray<TArray<FString>> OptionTags;

		Options.SetNum(NumOptions);
		OptionTags.SetNum(NumOptions);

		// Gather information about options, name and tags.
		for (int32 I = 0; I < NumOptions; ++I)
		{
			FString OptionName = PopulationClass->CustomizableObject->GetPrivate()->GetIntParameterAvailableOption(ParamIndex, I);
			Options[I] = OptionName;

			if (ParamTagsFound)
			{
				FFParameterOptionsTags* FoundOptionTags = ParamTagsFound->ParameterOptions.Find(OptionName);

				if (FoundOptionTags)
				{
					OptionTags[I] = FoundOptionTags->Tags;
				}

				OptionTags[I].Append(ParamTags);
			}

			// Add tag "none" to the none option
			//if (OptionName.Compare(FString("none"), ESearchCase::IgnoreCase) == 0)
			//{
			//	OptionTags[I].Add("none");
			//}
		}

		TArray<int32> ConstraintWeights;
		TArray<FConstraintIndex> SamplerIDs;

		bool bValidFound = false;

		// Overriden by characteristic weights if found
		FCustomizableObjectPopulationCharacteristic* CharacteristicFound = PopulationClass->Characteristics.FindByPredicate(
			[&IntParamName](FCustomizableObjectPopulationCharacteristic& C) { return C.ParameterName == IntParamName; });

		if (CharacteristicFound)
		{
			for (const FCustomizableObjectPopulationConstraint& Constraint : CharacteristicFound->Constraints)
			{
				TArray<int32> TagWeights;
				TagWeights.Init(0, NumOptions);

				// Global weights
				WeightParamOptionsTags(PopulationClass->Allowlist, PopulationClass->Blocklist, OptionTags, TagWeights);

				switch (Constraint.Type)
				{
				case EPopulationConstraintType::TAG:
				{
					WeightParamOptionsTags(Constraint.Allowlist, Constraint.Blocklist, OptionTags, TagWeights);
					break;
				}
				case EPopulationConstraintType::DISCRETE:
				{
					WeightParamCharacteristicDiscreteOptions(Constraint, TagWeights, Options);
					break;
				}
				default:
				{
					break;
				}
				}

				// Check at least one option is possible, if not, try to use the "None" option, if it's not possible add error and set all options equally probable.
				int32* ValidWeightFound = TagWeights.FindByPredicate([](int32 V) { return V > 0; });
				if (ValidWeightFound)
				{
					bValidFound = true;
					InOutGenerator.IntSamplers.Emplace(TagWeights, Options);
					SamplerIDs.Add(FConstraintIndex(InOutGenerator.IntSamplers.Num() - 1, EPopulationSamplerType::OPTION));
					ConstraintWeights.Add(Constraint.ConstraintWeight);
				}
			}
		}

		if (bValidFound)
		{
			// TODO: Optimization: if the same IntParameter has more than one IntSampler, add them all together to a single IntSampler with the option weights multiplied by their constraint weight
		}
		else
		{
			TArray<int32> TagWeights;
			TagWeights.Init(0, NumOptions);

			// Global weights
			WeightParamOptionsTags(PopulationClass->Allowlist, PopulationClass->Blocklist, OptionTags, TagWeights);
			int32* ValidWeightFound = TagWeights.FindByPredicate([](int32 V) { return V > 0; });
			if (!ValidWeightFound)
			{
				TagWeights.Init(1, TagWeights.Num());
			}

			InOutGenerator.IntSamplers.Emplace(TagWeights, Options);
			SamplerIDs.Add(FConstraintIndex(InOutGenerator.IntSamplers.Num() - 1, EPopulationSamplerType::OPTION));
			ConstraintWeights.Add(1);
		}

		InOutGenerator.ConstraintSamplers.Emplace(ConstraintWeights, IntParamName, SamplerIDs);
	}
}


FCustomizableObjectPopulationClassGenerator CompilePopulationClass(UCustomizableObjectPopulationClass* InPopulationClass)
{
	FCustomizableObjectPopulationClassGenerator ClassGenerator;

	if (!InPopulationClass)
	{
		return ClassGenerator;
	}

	UCustomizableObject* ClassObject = InPopulationClass->CustomizableObject;

	if (!ClassObject)
	{
		return ClassGenerator;
	}

	const int32 NumParameters = ClassObject->GetParameterCount();

	TArray<FString> BoolParamNames;
	TArray<FString> IntParamNames;
	TArray<FString> FloatParamNames;
	TArray<FString> ColorParamNames;

	for (int32 I = 0; I < NumParameters; ++I)
	{
		const EMutableParameterType ParamType = ClassObject->GetPrivate()->GetParameterType(I);

		switch (ParamType)
		{
			case EMutableParameterType::Bool:
			{
				BoolParamNames.Add(ClassObject->GetParameterName(I));
				break;
			}
			case EMutableParameterType::Int:
			{
				IntParamNames.Add(ClassObject->GetParameterName(I));
				break;
			}
			case EMutableParameterType::Float:
			{
				FloatParamNames.Add(ClassObject->GetParameterName(I));
				break;
			}
			case EMutableParameterType::Color:
			{
				ColorParamNames.Add(ClassObject->GetParameterName(I));
				break;
			}
		}
	}

	CompileBoolSamplers(BoolParamNames, InPopulationClass, ClassGenerator);
	CompileIntSamplers(IntParamNames, InPopulationClass, ClassGenerator);
	CompileFloatSamplers(FloatParamNames, InPopulationClass, ClassGenerator);
	CompileColorSamplers(ColorParamNames, InPopulationClass, ClassGenerator);

    return ClassGenerator;
}

#endif

#undef LOCTEXT_NAMESPACE
