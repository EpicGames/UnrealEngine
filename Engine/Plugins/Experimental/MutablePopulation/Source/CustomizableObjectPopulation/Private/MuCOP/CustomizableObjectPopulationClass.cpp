// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOP/CustomizableObjectPopulationClass.h"

#include "MuCOP/CustomizableObjectPopulationConstraint.h"
#include "MuCOP/CustomizableObjectPopulationCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectPopulationClass)

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClass"

using namespace CustomizableObjectPopulation;

void UCustomizableObjectPopulationClass::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FCustomizableObjectPopulationCustomVersion::GUID);
}

#undef LOCTEXT_NAMESPACE
