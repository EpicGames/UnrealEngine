// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOP/CustomizableObjectPopulationCustomVersion.h"

#include "Serialization/CustomVersion.h"

using namespace CustomizableObjectPopulation;

const FGuid FCustomizableObjectPopulationCustomVersion::GUID(0xC1270312, 0xAB730B1E, 0x0B1EC790, 0x917A7105);

// Register the custom version with core
FCustomVersionRegistration GRegisterCustomizableObjectPopulationCustomVersion(FCustomizableObjectPopulationCustomVersion::GUID, FCustomizableObjectPopulationCustomVersion::LatestVersion, TEXT("CustomizableObjectPopulationVer"));
