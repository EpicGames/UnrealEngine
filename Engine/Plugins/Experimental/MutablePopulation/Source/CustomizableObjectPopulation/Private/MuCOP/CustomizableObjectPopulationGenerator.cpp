// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOP/CustomizableObjectPopulationGenerator.h"

#include "Math/RandomStream.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCOP/CustomizableObjectPopulationCustomVersion.h"
#include "MuCOP/CustomizableObjectPopulationGeneratorPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectPopulationGenerator)

class FString;

void FCustomizableObjectPopulationClassGenerator::GenerateParameters(
	FRandomStream& Rand,
	UCustomizableObjectInstance* Instance)
{
	for (const FConstraintSampler& ConstraintSampler : ConstraintSamplers)
	{
		const FString& ParamName = ConstraintSampler.ParameterName;

		if (ConstraintSampler.IsValidSampler())
		{
			UCustomizableObject* CustomizableObject = Instance->GetCustomizableObject();

			if (!CustomizableObject)
			{
				return;
			}

			const FConstraintIndex& SamplerIndex = ConstraintSampler.GetSamplerID(ConstraintSampler.Sample(Rand));
			switch (SamplerIndex.SamplerType)
			{
			case EPopulationSamplerType::BOOL:
			{
				const bool SampledValue = BoolSamplers[SamplerIndex.SamplerIndex].Sample(Rand);

				if (Instance->ContainsBoolParameter(ParamName))
				{
					Instance->SetBoolParameterSelectedOption(ParamName, SampledValue);
				}

				break;
			}
			case EPopulationSamplerType::OPTION:
			{
				const FOptionSampler& IntParamSampler = IntSamplers[SamplerIndex.SamplerIndex];
				if (IntParamSampler.IsValidSampler())
				{
					const FString& OptionName = IntParamSampler.GetOptionName(IntParamSampler.Sample(Rand));
					if (Instance->ContainsIntParameter(ParamName))
					{
						if (!CustomizableObject->IsParameterMultidimensional(ParamName))
						{
							Instance->SetIntParameterSelectedOption(ParamName, OptionName);
						}
						else
						{
							// TODO: Randomize multidimensional integers
						}
					}
				}
				break;
			}
			case EPopulationSamplerType::UNIFORM_FLOAT:
			{
				const float SampledValue = UniformFloatSamplers[SamplerIndex.SamplerIndex].Sample(Rand);
				if (Instance->ContainsFloatParameter(ParamName))
				{
					if (!CustomizableObject->IsParameterMultidimensional(ParamName))
					{
						Instance->SetFloatParameterSelectedOption(ParamName, SampledValue);
					}
					else
					{
						// TODO: Randomize multidimensional floats
					}
				}
				break;
			}
			case EPopulationSamplerType::CURVE:
			{
				const float SampledValue = CurveSamplers[SamplerIndex.SamplerIndex].Sample(Rand);
				if (Instance->ContainsFloatParameter(ParamName))
				{
					Instance->SetFloatParameterSelectedOption(ParamName, SampledValue);
				}
				break;
			}
			case EPopulationSamplerType::RANGE:
			{
				const float SampledValue = RangesSamplers[SamplerIndex.SamplerIndex].Sample(Rand);
				if (Instance->ContainsFloatParameter(ParamName))
				{
					Instance->SetFloatParameterSelectedOption(ParamName, SampledValue);
				}
				break;
			}
			case EPopulationSamplerType::UNIFORM_CURVE_COLOR:
			{
				const FLinearColor SampledValue = UniformColorSamplers[SamplerIndex.SamplerIndex].Sample(Rand);
				if (Instance->ContainsVectorParameter(ParamName))
				{
					Instance->SetVectorParameterSelectedOption(ParamName, SampledValue);
				}
				break;
			}
			case EPopulationSamplerType::CONSTANT_COLOR:
			{
				if (Instance->ContainsVectorParameter(ParamName))
				{
					Instance->SetVectorParameterSelectedOption(ParamName, ConstantColors[SamplerIndex.SamplerIndex]);
				}
				break;
			}
			default:
			{
				break;
			}
			}
		}
	}
}

FArchive& operator<<(FArchive& Ar, FCustomizableObjectPopulationClassGenerator& ClassGenerator)
{
	Ar << ClassGenerator.BoolSamplers;
	Ar << ClassGenerator.IntSamplers;
	Ar << ClassGenerator.UniformFloatSamplers;
	Ar << ClassGenerator.RangesSamplers;
	Ar << ClassGenerator.CurveSamplers;
	Ar << ClassGenerator.UniformColorSamplers;
	Ar << ClassGenerator.ConstantColors;
	Ar << ClassGenerator.ConstraintSamplers;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCustomizableObjectPopulationGeneratorPrivate& Private)
{
	Ar << Private.ClassSampler;
	Ar << Private.bPopulationCompiledForCook;
	Ar << Private.ClassGenerators;

	return Ar;
}

UCustomizableObjectPopulationGenerator::UCustomizableObjectPopulationGenerator()
	: Private(MakeShareable(new FCustomizableObjectPopulationGeneratorPrivate()))
{
}

void UCustomizableObjectPopulationGenerator::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FCustomizableObjectPopulationCustomVersion::GUID);

	// Switch to standard serialization where possible and reorder layout so manually serialized properties are at the end
	if (Ar.CustomVer(FCustomizableObjectPopulationCustomVersion::GUID) < FCustomizableObjectPopulationCustomVersion::BeforeCustomVersionWasAdded)
	{
		check(Private);
		Ar << *Private;
		Ar << PopulationObjects;
	}
	else
	{
		Super::Serialize(Ar);
		check(Private);
		Ar << *Private;
	}
}

int32 UCustomizableObjectPopulationGenerator::GeneratePopulation(
	TArray<UCustomizableObjectInstance*>& OutInstances,
	int32 NumInstancesToGenerate) const
{
	const int32 Seed = FPlatformTime::Cycles();

	Generate(Seed, OutInstances, NumInstancesToGenerate);

	return Seed;
}

void UCustomizableObjectPopulationGenerator::RegeneratePopulation(
	int32 Seed,
	TArray<UCustomizableObjectInstance*>& OutInstances,
	int32 NumInstancesToGenerate) const
{
	Generate(Seed, OutInstances, NumInstancesToGenerate);
}

#if WITH_EDITOR
void UCustomizableObjectPopulationGenerator::Init(TArray<UCustomizableObject*> _PopulationObjects, TSharedPtr<FCustomizableObjectPopulationGeneratorPrivate> _Private)
{
	PopulationObjects.Empty();

	PopulationObjects = _PopulationObjects;
	Private = _Private;
	if (Private) {
		Private->bPopulationCompiledForCook = true;
	}
}

bool UCustomizableObjectPopulationGenerator::IsCompiledForCook() const
{
	return Private.IsValid() && Private->bPopulationCompiledForCook;
}
#endif

void UCustomizableObjectPopulationGenerator::Generate(
	int32 Seed,
	TArray<UCustomizableObjectInstance*>& OutPopulationInstances,
	int32 N) const
{
	check(Private);

	const int32 NumClasses = PopulationObjects.Num();

	TArray<int32> PopulationClassCount;
	PopulationClassCount.Init(0, NumClasses);

	FRandomStream ClassRand(Seed);

	// Determine how many intances of each class needs to be created.
	for (int32 I = 0; I < N; ++I)
	{
		const int32 ClassIndex = Private->ClassSampler.Sample(ClassRand);
		++PopulationClassCount[ClassIndex];
	}

	// Create Instances

	if (OutPopulationInstances.Num() == 0)
	{
		OutPopulationInstances.SetNum(N);
		for (UCustomizableObjectInstance*& Instance : OutPopulationInstances)
		{
			Instance = NewObject<UCustomizableObjectInstance>();
		}
	}

	// Needed to keep the index of the edited instance in the inner loops
	int InstanceIndex = 0;

	// Set Instance Objects.
	for (int32 I = 0; I < NumClasses; ++I)
	{
		UCustomizableObject* ClassObject = PopulationObjects[I];

		const int32 NumClassInstances = PopulationClassCount[I];
		for (int32 C = 0; C < NumClassInstances; ++C)
		{
			OutPopulationInstances[C + InstanceIndex]->SetObject(ClassObject);
		}

		InstanceIndex += NumClassInstances;
	}

	FRandomStream ParamRand(Seed);

	InstanceIndex = 0;

	// Generate Instance Parameters.
	for (int32 I = 0; I < NumClasses; ++I)
	{
		FCustomizableObjectPopulationClassGenerator& ClassGenerator = Private->ClassGenerators[I];

		const int32 NumClassInstances = PopulationClassCount[I];
		for (int32 C = 0; C < NumClassInstances; ++C)
		{
			ClassGenerator.GenerateParameters(ParamRand, OutPopulationInstances[C + InstanceIndex]);
		}

		InstanceIndex += NumClassInstances;
	}
}
