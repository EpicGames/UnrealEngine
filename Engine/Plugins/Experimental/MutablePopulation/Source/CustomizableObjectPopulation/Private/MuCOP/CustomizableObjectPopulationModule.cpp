// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Interfaces/IPluginManager.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/UnrealPortabilityHelpers.h"
#include "MuCOE/ICustomizableObjectPopulationModule.h"
#include "MuCOP/CustomizableObjectPopulation.h"
#include "MuCOP/CustomizableObjectPopulationGenerator.h"
#endif

/**
 * MovieSceneCore module implementation (private)
 */
class FCustomizableObjectPopulationModule : 
#if WITH_EDITOR
	public ICustomizableObjectPopulationModule
#else
	public IModuleInterface
#endif
{

public:

	// IModuleInterface 
	void StartupModule() override{}
	void ShutdownModule() override{}

#if WITH_EDITOR
	// ICustomizableObjectPopulationModule interface
	FString GetPluginVersion() const override;

	// Recompiles all the populations referenced in the Customizabled Object
	virtual void RecompilePopulations(UCustomizableObject* Object);
#endif

};

IMPLEMENT_MODULE( FCustomizableObjectPopulationModule, CustomizableObjectPopulation );

//-------------------------------------------------------------------------------------------------

#if WITH_EDITOR
FString FCustomizableObjectPopulationModule::GetPluginVersion() const
{
	FString Version = "x.x";
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin("MutablePopulation");
	if (Plugin.IsValid() && Plugin->IsEnabled())
	{
		Version = Plugin->GetDescriptor().VersionName;
	}
	return Version;
}

void FCustomizableObjectPopulationModule::RecompilePopulations(UCustomizableObject* Object)
{
	if (!Object)
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	
	// Getting all the referencers of the CO
	TArray<FName> COAssetNames;
	AssetRegistryModule.Get().GetReferencers(*Object->GetOuter()->GetPathName(), COAssetNames);
	
	if (COAssetNames.Num())
	{
		// Creating a filter to search only population classes
		FARFilter COFilter;
		UE_MUTABLE_GET_CLASSPATHS(COFilter).Add(UE_MUTABLE_GET_CLASS_PATHNAME(UCustomizableObjectPopulation::StaticClass()));

		for (const FName AssetName : COAssetNames)
		{
			COFilter.PackageNames.Add(AssetName);
		}

		//Getting population class assets
		TArray<FAssetData> PopulationAssetData;
		AssetRegistryModule.Get().GetAssets(COFilter, PopulationAssetData);

		for (int32 i = 0; i < PopulationAssetData.Num(); ++i)
		{
			if (UCustomizableObjectPopulation* Population = Cast<UCustomizableObjectPopulation>(PopulationAssetData[i].GetAsset()))
			{
				if (Population->IsValidPopulation())
				{
					Population->CompilePopulation();
				}
			}
		}
	}
}
#endif // WITH_EDITOR
