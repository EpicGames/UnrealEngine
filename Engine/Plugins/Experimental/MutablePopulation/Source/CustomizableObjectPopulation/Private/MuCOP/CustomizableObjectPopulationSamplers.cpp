// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOP/CustomizableObjectPopulationSamplers.h"

#include "Curves/CurveLinearColor.h"
#include "Math/RandomStream.h"
//#include "MuCOP/CustomizableObjectPopulation.h"
//#include "MuCOP/CustomizableObjectPopulationClass.h"

namespace
{
	TArray<int32> ComputeCumulativeWeights(const TArray<int32>& Weights)
	{
		TArray<int32> CumulativeWeights;

		const int32 NumWeights = Weights.Num();
		CumulativeWeights.SetNum(NumWeights + 1);

		CumulativeWeights[0] = -1;

		int32 C = 0;
		for (int32 I = 0; I < NumWeights; ++I)
		{
			// A negative weight is not valid. Clamp to 0.
			C += FMath::Max(Weights[I], 0);
			CumulativeWeights[I + 1] = C;
		}

		return CumulativeWeights;
	}
	/*
		TArray<int32> GetPopulationClassWeightsFromPairs(const TArray<FClassWeightPair>& Pairs)
		{
			TArray<int32> Weights;
			Weights.SetNum(Pairs.Num());

			for (int32 I = 0; I < Pairs.Num(); ++I)
			{
				Weights[I] = Pairs[I].ClassWeight;
			}

			return Weights;
		}

		TArray<int32> GetWeightsFormConstraintRanges(const TArray<FConstraintRanges>& Ranges)
		{
			TArray<int32> Weights;
			Weights.SetNum(Ranges.Num());

			for (int32 I = 0; I < Ranges.Num(); ++I)
			{
				Weights[I] = Ranges[I].RangeWeight;
			}

			return Weights;
		}

		TArray<TTuple<float, float>> GetValuesFromConstraintRanges(const TArray<FConstraintRanges>& Ranges)
		{
			TArray<TTuple<float, float>> Values;
			Values.SetNum(Ranges.Num());

			for (int32 I = 0; I < Ranges.Num(); ++I)
			{
				Values[I] = TTuple<float, float>{Ranges[I].MinimumValue, Ranges[I].MaximumValue};
			}

			return Values;
		}
		*/
} //namespace

namespace CustomizableObjectPopulation
{
	// FDiscreteImportanceSampler
	FDiscreteImportanceSampler::FDiscreteImportanceSampler(const TArray<int32>& Weights)
		: CumulativeWeights(ComputeCumulativeWeights(Weights))
	{
	}

	int32 FDiscreteImportanceSampler::Sample(FRandomStream& Rand) const
	{
		check(IsValidSampler());

		const int32 NumOptions = CumulativeWeights.Num() - 1;
		int32 R = Rand.RandRange(0, CumulativeWeights[NumOptions] - 1);

		int32 I = NumOptions;
		while (CumulativeWeights[--I] > R);

		return I;
	}

	bool FDiscreteImportanceSampler::IsValidSampler() const
	{
		return CumulativeWeights.Num() > 1 && CumulativeWeights.Last() > 0;
	}

	FArchive& operator<<(FArchive& Ar, FDiscreteImportanceSampler& Sampler)
	{
		Ar << Sampler.CumulativeWeights;

		return Ar;
	}
	////////////////

	// FOptionSampler
	FOptionSampler::FOptionSampler(const TArray<int32>& OptionWeights, const TArray<FString>& OptionNames)
		: FDiscreteImportanceSampler(OptionWeights)
		, OptionNames(OptionNames)
	{
	}

	const FString& FOptionSampler::GetOptionName(int32 OptionIndex) const
	{
		return OptionNames[OptionIndex];
	}

	FArchive& operator<<(FArchive& Ar, FOptionSampler& Sampler)
	{
		Ar << static_cast<FDiscreteImportanceSampler&>(Sampler);
		Ar << Sampler.OptionNames;

		return Ar;
	}

	////////////////

	// FBoolSampler
	FBoolSampler::FBoolSampler(int32 TrueWeight, int32 FalseWeight)
		: CumulativeValue(TrueWeight + FalseWeight)
		, TippingValue(TrueWeight)
	{
	}

	bool FBoolSampler::Sample(FRandomStream& Rand) const
	{
		const int32 R = Rand.RandRange(0, CumulativeValue - 1);

		return R < TippingValue;
	}

	bool FBoolSampler::IsValidSampler() const
	{
		return CumulativeValue > 0;
	}

	FArchive& operator<<(FArchive& Ar, FBoolSampler& Sampler)
	{
		Ar << Sampler.CumulativeValue;
		Ar << Sampler.TippingValue;

		return Ar;
	}

	////////////////


	// FRangesSampler
	FRangesSampler::FRangesSampler(const TArray<int32>& RangesWeights, const TArray<TTuple<float, float>>& RangesValues)
		: DiscreteSampler(RangesWeights)
		, RangeValues(RangesValues)
	{
	}

	float FRangesSampler::Sample(FRandomStream& Rand) const
	{
		check(IsValidSampler());

		int32 I = DiscreteSampler.Sample(Rand);

		return Rand.FRandRange(RangeValues[I].Get<0>(), RangeValues[I].Get<1>());
	}

	bool FRangesSampler::IsValidSampler() const
	{
		return DiscreteSampler.IsValidSampler();
	}

	FArchive& operator<<(FArchive& Ar, FRangesSampler& Sampler)
	{
		Ar << Sampler.DiscreteSampler;
		Ar << Sampler.RangeValues;

		return Ar;
	}

	////////////////


	// FFloatUniformSampler
	FFloatUniformSampler::FFloatUniformSampler(const float InMin, const float InMax)
		: MinValue(InMin), MaxValue(InMax)
	{
	}

	float FFloatUniformSampler::Sample(FRandomStream& Rand) const
	{
		return Rand.FRandRange(MinValue, MaxValue);
	}

	bool FFloatUniformSampler::IsValidSampler() const
	{
		return MinValue < MaxValue;
	}

	FArchive& operator<<(FArchive& Ar, FFloatUniformSampler& Sampler)
	{
		Ar << Sampler.MinValue;
		Ar << Sampler.MaxValue;

		return Ar;
	}

	////////////////

	// FCurveSampler
	FCurveSampler::FCurveSampler(const FRichCurve& InCurve, const int32 NumBins)
	{
		InCurve.CompressCurve(Curve);

		float MaxT;
		InCurve.GetTimeRange(MinT, MaxT);

		BinWidth = (MaxT - MinT) / float(NumBins);

		const float AreaSampleWidth = BinWidth / float(BinAreaSampleResolution);

		CumulativeBinWeights.SetNum(NumBins + 1);
		BinMaxHeights.SetNum(NumBins);

		CumulativeBinWeights[0] = -SMALL_NUMBER;

		float AbsoluteMax = 0.0f;
		// Compute cumulative bin weights. 
		float W = 0.0f;
		for (int32 B = 0; B < NumBins; ++B)
		{
			float T = MinT + BinWidth * float(B);

			// Mid point Integration.
			float MaxHeight = 0.0f;
			float Default = 0.0f;
			for (int32 A = 0; A < BinAreaSampleResolution; ++A)
			{
				const float S = FMath::Max(Curve.Eval(T + AreaSampleWidth * 0.5f), 0.0f);
				if (S > MaxHeight)
				{
					MaxHeight = S;
					Default = T + AreaSampleWidth * 0.5f;
				}
				W += AreaSampleWidth * S;

				T += AreaSampleWidth;
			}

			CumulativeBinWeights[B + 1] = W;
			BinMaxHeights[B] = MaxHeight;
			if (MaxHeight > AbsoluteMax)
			{
				AbsoluteMax = MaxHeight;
				MaxDefault = Default;
			}
		}
	}


	float FCurveSampler::Sample(FRandomStream& Rand) const
	{
		check(IsValidSampler());

		const int32 NumBins = CumulativeBinWeights.Num() - 1;

		const float CumulativeWeight = CumulativeBinWeights[NumBins];
		float R = Rand.FRandRange(0.0f, CumulativeWeight);

		// Could use binary search if the number of bins is large. In our use case
		// a linear search is good enough (possibly faster for a small set of bins)

		int32 I = NumBins;
		while (CumulativeBinWeights[--I] > R);

		// Enter only if we are not under the worst case scenario
		const float BinArea = I == 0 ? CumulativeBinWeights[I] : CumulativeBinWeights[I] - CumulativeBinWeights[I - 1];
		if (BinArea > 0.0001f)
		{
			const float BinMin = float(I) * BinWidth + MinT;
			const float BinMax = BinMin + BinWidth;
			const float BinHeight = BinMaxHeights[I];

			for (int tries = 0; tries < 1 << 16; tries++)
			{
				const float S = Rand.FRandRange(BinMin, BinMax);
				const float T = Rand.FRand() * BinHeight;

				const float V = Curve.Eval(S);
				if (V > T)
				{
					return S;
				}
			}
		}

		// Return the most probable value found as default, to make sure we don't return a value that is not legal
		return MaxDefault;
	}


	bool FCurveSampler::IsValidSampler() const
	{
		return CumulativeBinWeights.Num() > 1 && CumulativeBinWeights.Last() > 0.0f;
	}

	FArchive& operator<<(FArchive& Ar, FCurveSampler& Sampler)
	{
		Ar << Sampler.BinWidth;
		Ar << Sampler.MinT;
		Ar << Sampler.MaxDefault;
		Ar << Sampler.BinMaxHeights;
		Ar << Sampler.CumulativeBinWeights;
		Ar << Sampler.Curve;

		return Ar;
	}

	////////////////


	// FColorCurveUniformSampler
	FColorCurveUniformSampler::FColorCurveUniformSampler(UCurveLinearColor& InCurve, const float InMin, const float InMax)
		: MinValue(InMin), MaxValue(InMax)
	{
		for (uint8 i = 0; i < 4; ++i)
		{
			InCurve.FloatCurves[i].CompressCurve(ColorCurves[i]);
		}
	}

	FLinearColor FColorCurveUniformSampler::Sample(FRandomStream& Rand) const
	{
		const float randomPoint = Rand.FRandRange(MinValue, MaxValue);
		return FLinearColor(ColorCurves[0].Eval(randomPoint), ColorCurves[1].Eval(randomPoint), ColorCurves[2].Eval(randomPoint), ColorCurves[3].Eval(randomPoint));
	}

	bool FColorCurveUniformSampler::IsValidSampler() const
	{
		return MinValue < MaxValue;
	}

	FArchive& operator<<(FArchive& Ar, FColorCurveUniformSampler& Sampler)
	{
		Ar << Sampler.MinValue;
		Ar << Sampler.MaxValue;
		Ar << Sampler.ColorCurves[0];
		Ar << Sampler.ColorCurves[1];
		Ar << Sampler.ColorCurves[2];
		Ar << Sampler.ColorCurves[3];

		return Ar;
	}

	////////////////


	//FFloatSampler 
	/*
	FFloatSampler::FFloatSampler(FCurveSampler&& InSampler)
		: Sampler( TInPlaceType<FCurveSampler>{}, Forward<FCurveSampler>(InSampler) )
	{
	}

	FFloatSampler::FFloatSampler(FRangesSampler&& InSampler)
		: Sampler( TInPlaceType<FRangesSampler>{}, Forward<FRangesSampler>(InSampler) )
	{
	}

	FFloatSampler::FFloatSampler(FFloatUniformSampler&& InSampler)
		: Sampler( TInPlaceType<FFloatUniformSampler>{}, Forward<FFloatUniformSampler>(InSampler) )
	{
	}

	float FFloatSampler::Sample(FRandomStream& Rand) const
	{
		return Visit([&Rand](auto&& Arg) -> float { return Arg.Sample(Rand); }, Sampler);
	}

	bool FFloatSampler::IsValidSampler() const
	{
		return Visit([](auto&& Arg) -> bool { return Arg.IsValidSampler(); }, Sampler);
	}

	FArchive& operator<<(FArchive& Ar, FFloatSampler& Sampler)
	{
		using FloatSamplerType = FFloatSampler::FloatSamplerType;

		int32 TypeIndex = Sampler.Sampler.GetIndex();
		Ar << TypeIndex;

		// TODO: Could this be done better?
		if (Ar.IsLoading())
		{
			// Set Variant type from index if loading.
			switch (TypeIndex)
			{
				case FloatSamplerType::IndexOfType<FCurveSampler>() :
				{
					Sampler.Sampler.Set<FCurveSampler>(FCurveSampler());
					break;
				}
				case FloatSamplerType::IndexOfType<FRangesSampler>() :
				{
					Sampler.Sampler.Set<FRangesSampler>(FRangesSampler());
					break;
				}
				case FloatSamplerType::IndexOfType<FFloatUniformSampler>() :
				{
					Sampler.Sampler.Set<FFloatUniformSampler>(FFloatUniformSampler());
					break;
				}
				default:
				{
					check(false);
				}
			}
		}

		Visit([&Ar](auto&& Arg) -> void { Ar << Arg; }, Sampler.Sampler);

		return Ar;
	}
	*/

	// FPopulationClassSampler
	FPopulationClassSampler::FPopulationClassSampler(const TArray<int32>& Weights)
		: FDiscreteImportanceSampler(Weights)
	{
	}

	FArchive& operator<<(FArchive& Ar, FPopulationClassSampler& Sampler)
	{
		Ar << static_cast<FDiscreteImportanceSampler&>(Sampler);

		return Ar;
	}

	FArchive& operator<<(FArchive& Ar, FConstraintIndex& SamplerIndexes)
	{
		Ar << SamplerIndexes.SamplerIndex;
		Ar << SamplerIndexes.SamplerType;

		return Ar;
	}

	FConstraintSampler::FConstraintSampler(const TArray<int32>& OptionWeights, const FString& ParameterName, const TArray<FConstraintIndex>& Samplers)
		: FDiscreteImportanceSampler(OptionWeights)
		, ParameterName(ParameterName)
		, Samplers(Samplers)
	{
	}

	const FConstraintIndex& FConstraintSampler::GetSamplerID(int32 SamplerIndex) const
	{
		return Samplers[SamplerIndex];
	}

	FArchive& operator<<(FArchive& Ar, FConstraintSampler& Sampler)
	{
		Ar << static_cast<FDiscreteImportanceSampler&>(Sampler);
		Ar << Sampler.ParameterName;
		Ar << Sampler.Samplers;

		return Ar;
	}
} // namespace CustomizableObjectPopulation
////////////////
