// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectPopulation.generated.h"

class ITargetPlatform;
class UCustomizableObjectInstance;
class UCustomizableObjectPopulationClass;
class UCustomizableObjectPopulationGenerator;
struct FFrame;


USTRUCT()
struct FClassWeightPair
{
public:

	GENERATED_USTRUCT_BODY()

	FClassWeightPair() : 
		ClassWeight(1)
	{}

	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere, meta = (ToolTip = "Population Class that will randomply appear on Customizable Object Instances generated by this Population"))
	TObjectPtr<UCustomizableObjectPopulationClass> Class = nullptr;

	/** Influence of this class over all the classes of a population */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere, meta = (ClampMin = "1", ClampMax = "100.0", ToolTip = "How often will this class appear compared to other classes. Example: One class has weight 3, the other weight 1. The final chances are 75% of the first and 25% of the last."))
	int ClassWeight;

	friend FArchive& operator<<(FArchive& Ar, FClassWeightPair& ClassWeights)
	{
		Ar << ClassWeights.Class;
		Ar << ClassWeights.ClassWeight;

		return Ar;
	}
};

UCLASS(Blueprintable, BlueprintType)
class CUSTOMIZABLEOBJECTPOPULATION_API UCustomizableObjectPopulation : public UObject
{
public:

	GENERATED_BODY()

	UCustomizableObjectPopulation();

	/** Name of the customizable object population. */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere, BlueprintReadOnly)
	FString Name;

	/** List of Pupulation classes that define this population. */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TArray<FClassWeightPair> ClassWeights;
	
	/** Create a number of instances of the target population.
	 * @return the seed that was used to create instances in this call of the GeneratePopulation. Returns -1 if there in an error with the population asset.
	 */
	UFUNCTION(BlueprintPure, Category = "CustomizablePopulation")
	int32 GeneratePopulation(TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate = 1) const;

	/** Update the instances in the array with new parameter values for the target population.
	* @return false if there was an error with the population asset.
	*/
	UFUNCTION(BlueprintPure, Category = "CustomizablePopulation")
	bool RegeneratePopulation(int32 Seed, TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate = 1) const;
	
	/** Allows the compilation of this population only if all its classes exist */
	bool IsValidPopulation() const;

	bool HasGenerator() const;

#if WITH_EDITOR
public:
	void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	bool IsCachedCookedPlatformDataLoaded(const ITargetPlatform* TargetPlatform) override;

	void CompilePopulation(UCustomizableObjectPopulationGenerator* NewGenerator = nullptr);

private:

	void CompilePopulationInternal(UCustomizableObjectPopulation* Population);
#endif

private:

	UPROPERTY()
	TObjectPtr<UCustomizableObjectPopulationGenerator> Generator;

};
