// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CustomizableObjectPopulationClass.generated.h"

class FArchive;
struct FCustomizableObjectPopulationCharacteristic;


USTRUCT()
struct FPopulationClassParameterOptions
{
	GENERATED_USTRUCT_BODY()

	/** List of tags of a Parameter Option */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TArray<FString> Tags;
};

USTRUCT()
struct FPopulationClassParameter
{
public:

	GENERATED_USTRUCT_BODY()

	/** List of tags of a parameter */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TArray<FString> Tags;

	/** Map of options available for a parameter can have and their tags */
	UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	TMap<FString, FPopulationClassParameterOptions> ParameterOptions;
};

UCLASS()
class CUSTOMIZABLEOBJECTPOPULATION_API UCustomizableObjectPopulationClass : public UObject
{
public:

	GENERATED_BODY()

	void Serialize(FArchive& Ar) override;

	/** Name of the Customizable Object Population Class */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	FString Name;

	/** Customizable Object that defines this class */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	TObjectPtr<class UCustomizableObject> CustomizableObject;

	///** Map of parameters available for the Customizable Object and their tags */
	//UPROPERTY(Category = "CustomizablePopulation", EditAnywhere)
	//TMap<FString, FPopulationClassParameter> CustomizableObjectParameterTags;

	/** List of parameter tags available for this class */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	TArray<FString> Allowlist;
	
	/** List of parameter tags forbidden for this class */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	TArray<FString> Blocklist;

	/** Additional options to take into account for some parameters */
	UPROPERTY(Category = "CustomizablePopulationClass", EditAnywhere)
	TArray<FCustomizableObjectPopulationCharacteristic> Characteristics;

	/** Array with all the tags deffined for this population class */
	UPROPERTY()
	TArray<FString> Tags;
	


	/** Customizable Object Instance for UI purposes */
	TObjectPtr<class UCustomizableObjectInstance> CustomizableObjectInstance = nullptr;

	/** UCurve for UI purposes */
	TObjectPtr<class UCurveBase> EditorCurve;

};
