// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

struct FGuid;

namespace CustomizableObjectPopulation
{
	// Custom serialization version for assets/classes in the CustomizableObjectPopulation and CustomizableObjectPopulationEditor modules
	struct CUSTOMIZABLEOBJECTPOPULATION_API FCustomizableObjectPopulationCustomVersion
	{
		enum Type
		{
			// Before any version changes were made in the plugin
			BeforeCustomVersionWasAdded = 0,

			// -----<new versions can be added above this line>-------------------------------------------------
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};

		// The GUID for this custom version number
		const static FGuid GUID;

		private:
			FCustomizableObjectPopulationCustomVersion() {}
	};
} //namespace CustomizableObjectPopulation