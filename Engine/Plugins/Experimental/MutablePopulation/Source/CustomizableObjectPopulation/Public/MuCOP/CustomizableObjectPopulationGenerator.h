// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


//#include "MuCOP/CustomizableObjectPopulationSamplers.h"


#include "CustomizableObjectPopulationGenerator.generated.h"

class FArchive;
/*
struct FCustomizableObjectPopulationCharacteristic;
class UCustomizableObjectPopulation;

class UCustomizableObjectPopulationClass;
class UCustomizableObjectInstance;

template<class SamplerType>
struct TParameterSampler
{
	FString ParameterName;
	SamplerType Sampler;

	FArchive& operator<<(FArchive& Ar)
	{
		Ar << ParameterName;
		Ar << Sampler;

		return Ar;
	}
};

class FCustomizableObjectPopulationClassGenerator
{
	friend class FCustomizableObjectPopulationGeneratorCompiler;

private:
	//Bool samplers
	TArray<TParameterSampler<FBoolSampler>> BoolSamplers;

	//Int samplers
	TArray<TParameterSampler<FOptionSampler>> IntSamplers;

	// Float samplers
	TArray<TParameterSampler<FFloatUniformSampler>> UniformFloatSamplers;
	TArray<TParameterSampler<FRangesSampler>>       RangesSamplers;
	TArray<TParameterSampler<FCurveSampler>>        CurveSamplers;

public:
	void GenerateParameters( FRandomStream& Rand, UCustomizableObjectInstance* Instance );

	friend FArchive& operator<<( FArchive& Ar, FCustomizableObjectPopulationClassGenerator& ClassGenerator );
};
*/

class UCustomizableObject;
class UCustomizableObjectInstance;
// Forward declarations
struct FCustomizableObjectPopulationGeneratorPrivate;

UCLASS()
class CUSTOMIZABLEOBJECTPOPULATION_API UCustomizableObjectPopulationGenerator : public UObject
{
	GENERATED_BODY()

public:
	UCustomizableObjectPopulationGenerator();

	int32 GeneratePopulation( TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate = 1 ) const;
	
	void RegeneratePopulation( int32 Seed, TArray<UCustomizableObjectInstance*>& OutInstances, int32 NumInstancesToGenerate = 1 ) const;

	void Serialize( FArchive& Ar ) override;

#if WITH_EDITOR
	void Init(TArray<UCustomizableObject*> _PopulationObjects, TSharedPtr<FCustomizableObjectPopulationGeneratorPrivate> _Private);

	bool IsCompiledForCook() const;
#endif

private:

	UPROPERTY()
	TArray< TObjectPtr<UCustomizableObject> > PopulationObjects;

	TSharedPtr<FCustomizableObjectPopulationGeneratorPrivate> Private;

	void Generate( int32 Seed, TArray<UCustomizableObjectInstance*>& OutPopulationInstances, int32 N ) const;
};
