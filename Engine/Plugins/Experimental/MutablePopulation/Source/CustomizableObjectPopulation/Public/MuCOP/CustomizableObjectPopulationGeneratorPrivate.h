// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCOP/CustomizableObjectPopulationSamplers.h"

class UCustomizableObjectInstance;
using namespace CustomizableObjectPopulation;

class FCustomizableObjectPopulationClassGenerator
{
public:

	// Bool samplers
	TArray<FBoolSampler> BoolSamplers;

	// Int samplers
	TArray<FOptionSampler> IntSamplers;

	// Float samplers
	TArray<FFloatUniformSampler> UniformFloatSamplers;
	TArray<FRangesSampler>       RangesSamplers;
	TArray<FCurveSampler>        CurveSamplers;

	// Color samplers
	TArray<FColorCurveUniformSampler> UniformColorSamplers;
	TArray<FLinearColor> ConstantColors;

	// Characteristic samplers
	TArray<FConstraintSampler> ConstraintSamplers;

	void GenerateParameters(FRandomStream& Rand, UCustomizableObjectInstance* Instance);

	friend FArchive& operator<<(FArchive& Ar, FCustomizableObjectPopulationClassGenerator& ClassGenerator);
};


struct FCustomizableObjectPopulationGeneratorPrivate
{
	FPopulationClassSampler ClassSampler;
	bool bPopulationCompiledForCook = false;
	TArray<FCustomizableObjectPopulationClassGenerator> ClassGenerators;

	friend FArchive& operator<<( FArchive& Ar, FCustomizableObjectPopulationGeneratorPrivate& Private );
};

