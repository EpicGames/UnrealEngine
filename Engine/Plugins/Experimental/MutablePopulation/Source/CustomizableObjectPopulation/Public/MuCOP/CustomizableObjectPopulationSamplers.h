// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/RichCurve.h"

class FArchive;
class UCurveLinearColor;
struct FRandomStream;

namespace CustomizableObjectPopulation
{

class CUSTOMIZABLEOBJECTPOPULATION_API FDiscreteImportanceSampler
{
	TArray<int32> CumulativeWeights;

public:
	FDiscreteImportanceSampler(const TArray<int32>& Weights);
	FDiscreteImportanceSampler() = default;
	FDiscreteImportanceSampler(FDiscreteImportanceSampler&&) = default;
	FDiscreteImportanceSampler& operator=(const FDiscreteImportanceSampler&) = default;

	int32 Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FDiscreteImportanceSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FOptionSampler final : public FDiscreteImportanceSampler
{
	TArray<FString> OptionNames;

public:
	FOptionSampler(const TArray<int32>& OptionWeights, const TArray<FString>& OptionNames);
	FOptionSampler() = default;
	FOptionSampler(FOptionSampler&&) = default;

	const FString& GetOptionName(int32 OptionIndex) const;

	friend FArchive& operator<<(FArchive& Ar, FOptionSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FBoolSampler final
{
	int32 CumulativeValue = 0;
	int32 TippingValue = 0;

public:
	FBoolSampler(int32 TrueWeight = 1, int32 FalseWeight = 1);
	FBoolSampler(FBoolSampler&&) = default;

	bool Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FBoolSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FRangesSampler final
{
	FDiscreteImportanceSampler DiscreteSampler;
	TArray<TTuple<float, float>> RangeValues; // {RangeMin, RangeMax}

public:
	FRangesSampler(const TArray<int32>& RangesWeights, const TArray<TTuple<float, float>>& RangesValues);
	FRangesSampler() = default;
	FRangesSampler(FRangesSampler&&) = default;

	float Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FRangesSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FCurveSampler final
{
	static constexpr int32 BinAreaSampleResolution = 32;

	float BinWidth = 1.0f / BinAreaSampleResolution, MinT = 0.0f, MaxDefault = 0.0f;
	TArray<float> BinMaxHeights;
	TArray<float> CumulativeBinWeights;
	FCompressedRichCurve Curve;

public:
	FCurveSampler(const FRichCurve& InCurve, const int32 NumBins = BinAreaSampleResolution);
	FCurveSampler() = default;
	FCurveSampler(FCurveSampler&&) = default;

	float Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FCurveSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FFloatUniformSampler final
{
	float MinValue = 0.0f, MaxValue = 1.0f;

public:
	FFloatUniformSampler(const float InMin = 0.0f, const float InMax = 1.0f);
	FFloatUniformSampler(FFloatUniformSampler&&) = default;

	float Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FFloatUniformSampler& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FColorCurveUniformSampler final
{
	float MinValue = 0.0f, MaxValue = 1.0f;
	FCompressedRichCurve ColorCurves[4];

public:
	FColorCurveUniformSampler(UCurveLinearColor& InCurve, const float InMin = 0.0f, const float InMax = 1.0f);
	FColorCurveUniformSampler() = default;
	FColorCurveUniformSampler(FColorCurveUniformSampler&&) = default;

	FLinearColor Sample(FRandomStream& Rand) const;
	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FColorCurveUniformSampler& Sampler);
};


/*
class FFloatSampler final
{
	using FloatSamplerType = TVariant<FCurveSampler, FRangesSampler, FFloatUniformSampler>;
	FloatSamplerType Sampler;

public:
	FFloatSampler(FCurveSampler&& InSampler);
	FFloatSampler(FRangesSampler&& InSampler);
	FFloatSampler(FFloatUniformSampler&& InSampler);

	float Sample(FRandomStream& Rand) const;

	bool IsValidSampler() const;

	friend FArchive& operator<<(FArchive& Ar, FFloatSampler& Sampler);
};
*/

class CUSTOMIZABLEOBJECTPOPULATION_API FPopulationClassSampler final : public FDiscreteImportanceSampler
{
public:
	FPopulationClassSampler(const TArray<int32> & Weights);
	FPopulationClassSampler() = default;
	FPopulationClassSampler(FPopulationClassSampler&&) = default;
	FPopulationClassSampler& operator=(const FPopulationClassSampler&) = default;
	friend FArchive& operator<<(FArchive& Ar, FPopulationClassSampler& Sampler);
};

enum class EPopulationSamplerType : uint8
{
	NONE,
	BOOL,
	OPTION,
	UNIFORM_FLOAT,
	RANGE,
	CURVE,
	UNIFORM_CURVE_COLOR,
	CONSTANT_COLOR
};

struct CUSTOMIZABLEOBJECTPOPULATION_API FConstraintIndex
{
public:

	FConstraintIndex() :
	SamplerIndex(-1),
	SamplerType(EPopulationSamplerType::NONE)
	{}
	FConstraintIndex(const int32 SamplerIndex, const EPopulationSamplerType SamplerType) :
		SamplerIndex(SamplerIndex),
		SamplerType(SamplerType)
	{}

	int32 SamplerIndex;
	EPopulationSamplerType SamplerType;

	friend FArchive& operator<<(FArchive& Ar, FConstraintIndex& Sampler);
};

class CUSTOMIZABLEOBJECTPOPULATION_API FConstraintSampler final : public FDiscreteImportanceSampler
{
public:
	FString ParameterName;
	TArray<FConstraintIndex> Samplers;

	FConstraintSampler(const TArray<int32>& OptionWeights, const FString& ParameterName, const TArray<FConstraintIndex>& Samplers);
	FConstraintSampler() = default;
	FConstraintSampler(FConstraintSampler&&) = default;

	const FConstraintIndex& GetSamplerID(int32 SamplerIndex) const;

	friend FArchive& operator<<(FArchive & Ar, FConstraintSampler& Sampler);
};

} //namespace CustomizableObjectPopulation
