// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class CustomizableObjectPopulationEditor : ModuleRules
{

	public CustomizableObjectPopulationEditor(ReadOnlyTargetRules TargetRules) : base(TargetRules)
	{
		ShortName = "MuCOPE";

		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				
				"Slate",
				"SlateCore",
				"RenderCore",
				"RHI",
				"UnrealEd",
				"TargetPlatform",
				"RawMesh",
				"LevelEditor",
				"AssetTools",
				"GraphEditor",
				"InputCore",
				"Kismet",
				"AdvancedPreviewScene",
				"AppFramework",
				"Projects",
				"ClothingSystemRuntimeCommon",

				"MutableRuntime",
				"MutableTools",
				
				"ContentBrowser",
				"AssetDefinition",
				"ToolMenus",
				"WorkspaceMenuStructure"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"SceneOutliner",
				"ClassViewer",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"CustomizableObject",
				"CustomizableObjectEditor",
				"CustomizableObjectPopulation",
			}
		);
	}
}
