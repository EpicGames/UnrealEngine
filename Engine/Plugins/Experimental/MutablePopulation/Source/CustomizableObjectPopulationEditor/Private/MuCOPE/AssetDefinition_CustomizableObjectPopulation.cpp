// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/AssetDefinition_CustomizableObjectPopulation.h"

#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOP/CustomizableObjectPopulation.h"
#include "MuCOPE/CustomizableObjectPopulationEditorModule.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulation"


TConstArrayView<FAssetCategoryPath> UAssetDefinition_CustomizableObjectPopulation::GetAssetCategories() const
{
	static const std::initializer_list<FAssetCategoryPath> Categories =
	{
		// Asset can be found inside the Mutable submenu 
		NSLOCTEXT("AssetTypeActions", "Mutable", "Mutable")
	};
	return Categories;
}

EAssetCommandResult UAssetDefinition_CustomizableObjectPopulation::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	
	for (UCustomizableObjectPopulation* Object : OpenArgs.LoadObjects<UCustomizableObjectPopulation>())
	{
		ICustomizableObjectPopulationEditorModule* CustomizableObjectPopulationEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectPopulationEditorModule>( "CustomizableObjectPopulationEditor" );
		CustomizableObjectPopulationEditorModule->CreateCustomizableObjectPopulationEditor(Mode, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}

namespace MenuExtension_CustomizableObjectPopulation
{
	void ExecRecompilePopulations(const FToolMenuContext& ToolMenuContext)
	{
		const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(ToolMenuContext);
		check(Context);

		// Recompile all selected Populations
		for (UCustomizableObjectPopulation* Population : Context->LoadSelectedObjects<UCustomizableObjectPopulation>())
		{
			if (Population && Population->IsValidPopulation())
			{
				Population->CompilePopulation();
			}
		}
	}
	
	// Method that registers the callbacks to be executed and the buttons to be displayed when right-clicking an object
	// of the CustomizableObject population type.
	static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{
	   	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
	   	{
			FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
			UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(UCustomizableObjectPopulation::StaticClass());
		
			FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
			Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				// Here add the actions you want to be able to perform
	
				// Recompile Populations
				{
					const TAttribute<FText> Label = NSLOCTEXT("UAssetDefinition_CustomizableObjectPopulation","CustomizableObjectPopulation_RecompilePopulations","Recompile Populations");
					const TAttribute<FText> ToolTip = NSLOCTEXT("UAssetDefinition_CustomizableObjectPopulation", "CustomizableObjectPopulation_RecompilePopulationsToolTip", "Recompiles all the Mutable Populations Assets.");
					const FSlateIcon Icon = FSlateIcon();
	
					FToolUIAction UIAction;
					UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecRecompilePopulations);
					InSection.AddMenuEntry("CustomizableObject_ExecuteEdit", Label, ToolTip, Icon, UIAction);
				}
				
			}));
	   	}));
   });
}


#undef LOCTEXT_NAMESPACE

