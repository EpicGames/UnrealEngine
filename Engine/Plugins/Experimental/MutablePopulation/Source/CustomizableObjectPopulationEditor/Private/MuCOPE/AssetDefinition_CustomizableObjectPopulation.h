// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "MuCOP/CustomizableObjectPopulation.h"

#include "AssetDefinition_CustomizableObjectPopulation.generated.h"


UCLASS()
class UAssetDefinition_CustomizableObjectPopulation final : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:
	
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CustomizableObjectPopulation", "Customizable Population");};

	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(0, 0, 0));};

	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCustomizableObjectPopulation::StaticClass();}

	
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};
