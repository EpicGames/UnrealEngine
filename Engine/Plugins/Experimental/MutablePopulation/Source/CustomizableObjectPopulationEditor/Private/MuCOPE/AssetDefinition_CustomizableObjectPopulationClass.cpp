// Copyright Epic Games, Inc. All Rights Reserved.


#include "MuCOPE/AssetDefinition_CustomizableObjectPopulationClass.h"

#include "MuCO/ICustomizableObjectEditorModule.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"
#include "MuCOPE/CustomizableObjectPopulationEditorModule.h"


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClass"


TConstArrayView<FAssetCategoryPath> UAssetDefinition_CustomizableObjectPopulationClass::GetAssetCategories() const
{
	static const std::initializer_list<FAssetCategoryPath> Categories =
	{
		// Asset can be found inside the Mutable submenu 
		NSLOCTEXT("AssetTypeActions", "Mutable", "Mutable")
	};
	return Categories;
}

EAssetCommandResult UAssetDefinition_CustomizableObjectPopulationClass::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	const EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
	
	for (UCustomizableObjectPopulationClass* Object : OpenArgs.LoadObjects<UCustomizableObjectPopulationClass>())
	{
		ICustomizableObjectPopulationEditorModule* CustomizableObjectPopulationEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectPopulationEditorModule>("CustomizableObjectPopulationEditor");
		CustomizableObjectPopulationEditorModule->CreateCustomizableObjectPopulationClassEditor(Mode, OpenArgs.ToolkitHost, Object);
	}

	return EAssetCommandResult::Handled;
}


#undef LOCTEXT_NAMESPACE
