// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinitionDefault.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"

#include "AssetDefinition_CustomizableObjectPopulationClass.generated.h"


UCLASS()
class UAssetDefinition_CustomizableObjectPopulationClass final : public UAssetDefinitionDefault
{
	GENERATED_BODY()
	
public:

	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_CustomizableObjectPopulationClass", "Customizable Population Class");};

	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(255, 255, 255));};

	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCustomizableObjectPopulationClass::StaticClass();};

	
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;

	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
};
