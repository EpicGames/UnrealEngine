// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationClassDetails.h"

#include "Curves/CurveLinearColor.h"
#include "DetailLayoutBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IDetailsView.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCOP/CustomizableObjectPopulationCharacteristic.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"
#include "MuCOP/CustomizableObjectPopulationConstraint.h"
#include "PropertyCustomizationHelpers.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextComboBox.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
class SWidget;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClassDetails"

// Line and text Colors
const FColor OddLineColor = FColor(200, 245, 255, 255);
const FSlateColor OddTextColor = FSlateColor(FLinearColor(.7f, .96f, 1.0f, 1.0f));
const FColor EvenLineColor = FColor(245, 245, 200, 255);
const FSlateColor EvenTextColor = FSlateColor(FLinearColor(.96f, .96f, .78f, 1.0f));

TSharedRef<IDetailCustomization> FCustomizableObjectPopulationClassDetails::MakeInstance()
{
	return MakeShareable(new FCustomizableObjectPopulationClassDetails);
}

void FCustomizableObjectPopulationClassDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	PopulationClass = nullptr;
	DetailBuilderPtr = &DetailBuilder;

	TSharedPtr<const IDetailsView> DetailsView = DetailBuilder.GetDetailsViewSharedPtr();
	if (DetailsView && DetailsView->GetSelectedObjects().Num())
	{
		PopulationClass = Cast<UCustomizableObjectPopulationClass>(DetailsView->GetSelectedObjects()[0].Get());
	}

	TSharedPtr<IPropertyHandle> CustomizableObjectProperty = DetailBuilder.GetProperty("CustomizableObject");
	CustomizableObjectProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FCustomizableObjectPopulationClassDetails::OnCustomizableObjectPropertyChanged));

	// Hiding all properties that have their own widget
	DetailBuilder.HideProperty("CustomizableObjectParameterTags");
	DetailBuilder.HideProperty("Allowlist");
	DetailBuilder.HideProperty("Blocklist");
	DetailBuilder.HideProperty("Characteristics");

	// Cretaing new categories to distribute properties
	IDetailCategoryBuilder& ClassBasics = DetailBuilder.EditCategory("CustomizablePopulationClass");
	IDetailCategoryBuilder& TagManagementCategory = DetailBuilder.EditCategory("Class-widePopulationTagLists");
	IDetailCategoryBuilder& CharacteristicsCategory = DetailBuilder.EditCategory("ClassCharacteristics");

	if (PopulationClass && PopulationClass->CustomizableObject && PopulationClass->CustomizableObject->IsCompiled())
	{
		// Tag Management widgets ( Tag manager, withelist and Blocklist)
		TagManagementWidgets = SNew(SPopulationClassTagManager)
		.PopulationClass(PopulationClass)
		.DetailBuilderPtr(DetailBuilderPtr)
		.AllowlistPtr(&PopulationClass->Allowlist)
		.BlocklistPtr(&PopulationClass->Blocklist);

		TagManagementCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
		[
			TagManagementWidgets.ToSharedRef()
		];

		// Characteristics widgets (Constraints)
		BuildCharacteristicsWidgets();
		CharacteristicsCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
		[
			CharacteristicsWidgets.ToSharedRef()
		];
	}
	else
	{
		if (!PopulationClass || !PopulationClass->CustomizableObject)
		{
			TagManagementCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CustomizableObjectSelectionErrorTag","Select a Customizable Object"))
			];

			CharacteristicsCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CustomizableObjectSelectionErrorCharacteristic", "Select a Customizable Object"))
			];
		}
		else
		{
			TagManagementCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CustomizableObjectCompilationErrorTag", "Compile the Customizable Object and restart the Population Class Editor"))
			];

			CharacteristicsCategory.AddCustomRow(LOCTEXT("FCustomizableObjectPopulationClassDetails", "Blocks"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CustomizableObjectCompilationErrorCharacteristic", "Compile the Customizable Object and restart the Population Class Editor"))
			];
		}
	}
}


void FCustomizableObjectPopulationClassDetails::OnCustomizableObjectPropertyChanged()
{
	if (DetailBuilderPtr)
	{
		// Because most of the properties and tags are related with the game object,
		// if we change it we will reset the populetion class properties
		PopulationClass->Tags.Empty();
		PopulationClass->Blocklist.Empty();
		PopulationClass->Allowlist.Empty();
		PopulationClass->Characteristics.Empty();
		DetailBuilderPtr->ForceRefreshDetails();
	}
}


void FCustomizableObjectPopulationClassDetails::BuildCharacteristicsWidgets()
{
	CharacteristicsWidgets = SNew(SVerticalBox);

	CharacteristicsWidgets->AddSlot()
	.AutoHeight()
	.Padding(0.0f,5.0f,0.0f,0.0f)
	[
		SNew(SHorizontalBox)
		
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.OnClicked(this, &FCustomizableObjectPopulationClassDetails::OnAddCharacteristicButtonPessed)
			.ToolTipText(LOCTEXT("AddCharacteristic", "Add a New Characteristic"))
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddCharacteristicText", "+ Add Characteristic"))
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
			]
		]
	];

	// Creating a widget for each characteristic
	for (int32 i = 0; i < PopulationClass->Characteristics.Num(); ++i)
	{
		CharacteristicsWidgets->AddSlot()
		.AutoHeight()
		[
			SNew(SPopulationClassCharacteristic)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.CharactericticIndex(i)
		];	
	}
}


FReply FCustomizableObjectPopulationClassDetails::OnAddCharacteristicButtonPessed()
{
	PopulationClass->Characteristics.Add(FCustomizableObjectPopulationCharacteristic());
	PopulationClass->Characteristics.Last().Constraints.Add(FCustomizableObjectPopulationConstraint());
	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


// Tag Manager Widget -------------------------

void SPopulationClassTagManager::Construct(const FArguments& InArgs)
{
	PopulationClass	= InArgs._PopulationClass;
	DetailBuilderPtr = InArgs._DetailBuilderPtr;
	AllowlistPtr = InArgs._AllowlistPtr;
	BlocklistPtr = InArgs._BlocklistPtr;

	BuildAllowlistWidget();
	BuildBlocklistWidget();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 10.0f, 0.0f, 0.0f)
		[
			AllowlistWidget.ToSharedRef()
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(15.0f, 0.0f, 15.0f, 0.0f)
		[
			SNew(SCustomEditorLine)
			.Horizontal(false)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 10.0f, 0.0f, 0.0f)
		[
			BlocklistWidget.ToSharedRef()
		]
	];
}


void SPopulationClassTagManager::BuildAllowlistWidget()
{
	AllowlistWidget = SNew(SVerticalBox);

	AllowlistWidget->AddSlot()
	.AutoHeight()
	.Padding(0.0f,0.0f,0.0f,5.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("Allow List - %d"), AllowlistPtr->Num())))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(20.0f,0.0f,0.0f,0.0f)
		[
			SNew(SButton)
			.OnClicked(this, &SPopulationClassTagManager::OnAddTagButtonPressed, true)
			.ToolTipText(LOCTEXT("AddAllowlistTag", "Add Allow List Tag"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Plus")))
			]
		]
	];

	AllowlistWidget->AddSlot()
	.AutoHeight()
	.Padding(-5.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SCustomEditorLine)
		.Length(120.0f)
	];

	// Creating a taglist widget for each tag of the allow list
	for (int32 i = 0; i < AllowlistPtr->Num(); ++i)
	{
		AllowlistWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SPopulationClassTagList)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.TagIndex(i)
			.ListPtr(AllowlistPtr)
		];
	}

}


void SPopulationClassTagManager::BuildBlocklistWidget()
{
	BlocklistWidget = SNew(SVerticalBox);

	BlocklistWidget->AddSlot()
	.AutoHeight()
	.Padding(0.0f, 0.0f, 0.0f, 5.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("Block List - %d"), BlocklistPtr->Num())))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(20.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.OnClicked(this, &SPopulationClassTagManager::OnAddTagButtonPressed, false)
			.ToolTipText(LOCTEXT("AddBlocklistTag", "Add Block List Tag"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Plus")))
			]
		]
	];


	BlocklistWidget->AddSlot()
	.AutoHeight()
	.Padding(-5.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SCustomEditorLine)
		.Length(120.0f)
	];

	// Creating a taglist widget for each tag of the block list
	for (int32 i = 0; i < BlocklistPtr->Num(); ++i)
	{
		BlocklistWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 10.0f, 0.0f, 5.0f)
		[
			SNew(SPopulationClassTagList)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.TagIndex(i)
			.ListPtr(BlocklistPtr)
		];
	}
}


FReply SPopulationClassTagManager::OnAddTagButtonPressed(bool bIsAllowlist)
{
	if (bIsAllowlist)
	{
		AllowlistPtr->Add("");
	}
	else
	{
		BlocklistPtr->Add("");
	}

	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


// List Widget -------------------------

void SPopulationClassTagList::Construct(const FArguments& InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	DetailBuilderPtr = InArgs._DetailBuilderPtr;
	TagIndex = InArgs._TagIndex;
	ListPtr = InArgs._ListPtr;
	TagValue = (*ListPtr)[TagIndex];

	TSharedPtr<FString> SelectedTag;
	AllTags.Empty();

	AllTags.Add(MakeShareable(new FString("None")));
	SelectedTag = AllTags.Last();

	// bool to determine if the tag has been deleted
	bool bTagFound = false;

	// Filling combobox options
	for (int32 i = 0; i < PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Num(); ++i)
	{
		AllTags.Add(MakeShareable(new FString(PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags()[i])));

		if (PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags()[i].Equals(TagValue))
		{
			SelectedTag = AllTags.Last();
			bTagFound = true;
		}
	}

	
	TSharedPtr<SHorizontalBox> Content = SNew(SHorizontalBox);

	if (bTagFound)
	{
		Content->AddSlot()
		.Padding(10.0f, 0.0f, 0.0f, 0.0f)
		.AutoWidth()
		[
			SAssignNew(TagsComboBox, STextComboBox)
			.OptionsSource(&AllTags)
			.InitiallySelectedItem(SelectedTag)
			.OnSelectionChanged(this, &SPopulationClassTagList::OnComboBoxSelectionChanged)
		];
	}
	else
	{
		FText TagToolTip = LOCTEXT("NoneTagWarning", "The None tag will be ignored.");

		if (!TagValue.IsEmpty() && !TagValue.Contains("None"))
		{
			// Setting the SelectedTag with the deleted tag
			SelectedTag = MakeShareable(new FString(TagValue));
			TagToolTip = LOCTEXT("DeletedTagWarning", "This Tag will be ingored, it's None or maybe it has been removed from the CO.");
		}
		
		Content->AddSlot()
		.Padding(10.0f, 0.0f, 0.0f, 0.0f)
		.AutoWidth()
		[
			SAssignNew(TagsComboBox, STextComboBox)
			.OptionsSource(&AllTags)
			.InitiallySelectedItem(SelectedTag)
			.OnSelectionChanged(this, &SPopulationClassTagList::OnComboBoxSelectionChanged)
			.ColorAndOpacity(FSlateColor(FLinearColor::Red))
			.ToolTipText(TagToolTip)
		];
	}

	Content->AddSlot()
	.AutoWidth()
	.Padding(5.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SButton)
		.OnClicked(this, &SPopulationClassTagList::OnRemoveTagButtonPressed, TagIndex)
		.ToolTipText(LOCTEXT("RemoveTag", "Remove Tag"))
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush(TEXT("Cross")))
		]
	];


	ChildSlot
	[
		Content.ToSharedRef()
	];
}


void SPopulationClassTagList::OnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{	
		if (*Selection == "None")
		{
			(*ListPtr)[TagIndex].Empty();
		}
		else
		{
			(*ListPtr)[TagIndex] = *Selection;
		}

		PopulationClass->MarkPackageDirty();
		DetailBuilderPtr->ForceRefreshDetails();
	}
}


FReply SPopulationClassTagList::OnRemoveTagButtonPressed(int32 Index)
{
	ListPtr->RemoveAt(TagIndex);
	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


// Characteristic Widget -------------------------

void SPopulationClassCharacteristic::Construct(const FArguments& InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	DetailBuilderPtr = InArgs._DetailBuilderPtr;
	CharactericticIndex = InArgs._CharactericticIndex;

	TSharedPtr<FString> SelectedParameter;
	CustomizableObjectParameters.Empty();

	CustomizableObjectParameters.Add(MakeShareable(new FString("None")));
	SelectedParameter = CustomizableObjectParameters.Last();

	int32 ParameterIndex = -1;
	
	TSet<FString> UsedParameters;
	
	// Inidcates that this parameter still exist
	bool bParameterDeleted = false;

	if (!PopulationClass || !PopulationClass->Characteristics.IsValidIndex(CharactericticIndex))
	{
		return;
	}

	// Create Map/Set with all the used parameters in the characteristics
	for (int32 i = 0; i < PopulationClass->Characteristics.Num(); ++i)
	{
		UsedParameters.Add(PopulationClass->Characteristics[i].ParameterName);
	}

	// Filling combobox options
	if (PopulationClass->CustomizableObject)
	{
		for (int32 i = 0; i < PopulationClass->CustomizableObject->GetParameterCount(); ++i)
		{
			FString ParameterName = PopulationClass->CustomizableObject->GetParameterName(i);

			// Do not add the parameter in the combo box if it's in another characteristic
			if (!UsedParameters.Contains(ParameterName))
			{
				CustomizableObjectParameters.Add(MakeShareable(new FString(ParameterName)));
			}
			
			// Add the parameter this is the characteristic with this parameter
			if (PopulationClass->Characteristics[CharactericticIndex].ParameterName == ParameterName)
			{
				CustomizableObjectParameters.Add(MakeShareable(new FString(ParameterName)));
				SelectedParameter = CustomizableObjectParameters.Last();

				// Keep the index of the parameter
				ParameterIndex = i;
			}
		}
	}

	// The selected Parameter has been deleted
	if (ParameterIndex == -1 && PopulationClass->Characteristics[CharactericticIndex].ParameterName.IsEmpty() == false)
	{
		CustomizableObjectParameters.Add(MakeShareable(new FString(PopulationClass->Characteristics[CharactericticIndex].ParameterName)));
		SelectedParameter = CustomizableObjectParameters.Last();
		bParameterDeleted = true;
	}

	// Sort Parameters by name
	CustomizableObjectParameters.Sort
	(
		[](const TSharedPtr<FString> A, const TSharedPtr<FString> B)
		{ 
			return (*A).Compare(*B) < 0; 
		}
	);

	// Constraints container widget
	ConstraintsWidgets = SNew(SVerticalBox);

	for (int32 i = 0; i < PopulationClass->Characteristics[CharactericticIndex].Constraints.Num(); ++i)
	{
		ConstraintsWidgets->AddSlot()
		.AutoHeight()
		.Padding(10.0f, 5.0f, 0.0f, 5.0f)
		[
			SNew(SCustomEditorLine)
			.Length(300.0f)
		];
		
		ConstraintsWidgets->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 5.0f)
		[
			SNew(SPopulationClassConstraint)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.CharactericticIndex(CharactericticIndex)
			.ConstraintIndex(i)
			.ParameterIndex(ParameterIndex)
		];
	}


	TSharedPtr<SHorizontalBox> ConstraintsWidget = SNew(SHorizontalBox);

	if (ParameterIndex != -1)
	{
		bool bAcceptsMulticonstraints = false;
		if (PopulationClass->CustomizableObject) {
			const EMutableParameterType CharacteristicType = PopulationClass->CustomizableObject->GetParameterTypeByName(PopulationClass->Characteristics[CharactericticIndex].ParameterName);
			bAcceptsMulticonstraints = CharacteristicType == EMutableParameterType::Int || CharacteristicType == EMutableParameterType::Float;
		}

		if (bAcceptsMulticonstraints)
		{
			ConstraintsWidget->AddSlot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("Constraints: %d"), PopulationClass->Characteristics[CharactericticIndex].Constraints.Num())))
				.ColorAndOpacity(bParameterDeleted ? FSlateColor(FLinearColor::Red) : (CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor))
			];

			ConstraintsWidget->AddSlot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.OnClicked(this, &SPopulationClassCharacteristic::OnAddConstraintButtonPessed)
				.ToolTipText(LOCTEXT("AddConstraint", "Add a New Constraint"))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Plus")))
				]
			];
		}
	}
	else
	{
		if (bParameterDeleted)
		{
			ConstraintsWidgets->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("This Parameter has been deleted from the CO."))))
				.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.2f, 0.2f, 1.0f)))
			];
		}
		else
		{
			// If all the parameters have a characteristic
			if (CustomizableObjectParameters.Num() == 1)
			{
				ConstraintsWidgets->AddSlot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 5.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::Printf(TEXT("There are no more parameters to modify"))))
					.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.2f, 0.2f, 1.0f)))
				];
			}
			else
			{
				ConstraintsWidgets->AddSlot()
				.AutoHeight()
				.Padding(0.0f, 5.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("NoParameterSelectedText", "Select a Parameter to add a Constraint"))
				];
			}
		}
	}


	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 15.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("Characteristic %d - Parameter:"), CharactericticIndex+1)))
				.ColorAndOpacity(bParameterDeleted ? FSlateColor(FLinearColor::Red) : (CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(STextComboBox)
				.OptionsSource(&CustomizableObjectParameters)
				.InitiallySelectedItem(SelectedParameter)
				.OnSelectionChanged(this, &SPopulationClassCharacteristic::OnParameterSelectionChanged)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("RemoveCharacteristic", "Remove Characteristic"))
				.OnClicked(this, &SPopulationClassCharacteristic::OnRemoveCharacteristicButtonPressed)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]

			+ SHorizontalBox::Slot()
			.Padding(10.0f, 8.0f, 0.0f, 0.0f)
			[
				SNew(SCustomEditorLine)
				.LineColor(bParameterDeleted ? FColor::Red : (CharactericticIndex % 2 == 0 ? EvenLineColor : OddLineColor))
			]

		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(15.0f, 5.0f, 0.0f, 0.0f)
		[
			ConstraintsWidget.ToSharedRef()
		]
		
		+ SVerticalBox::Slot()
		.Padding(30.0f, 5.0f, 0.0f, 5.0f)
		.AutoHeight()
		[
			ConstraintsWidgets.ToSharedRef()
		]

		+ SVerticalBox::Slot()
		.Padding(0.0f, 0.0f, 0.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SCustomEditorLine)
			.LineColor(bParameterDeleted ? FColor::Red : (CharactericticIndex % 2 == 0 ? EvenLineColor : OddLineColor))
		]
	];

}


void SPopulationClassCharacteristic::OnParameterSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		PopulationClass->Characteristics[CharactericticIndex].ParameterName = *Selection;
		
		// Now we do not support more than one constraint
		PopulationClass->Characteristics[CharactericticIndex].Constraints[0].Type = EPopulationConstraintType::NONE;
		
		PopulationClass->MarkPackageDirty();
		DetailBuilderPtr->ForceRefreshDetails();
	}
}


FReply SPopulationClassCharacteristic::OnAddConstraintButtonPessed()
{
	PopulationClass->Characteristics[CharactericticIndex].Constraints.Add(FCustomizableObjectPopulationConstraint());
	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


FReply SPopulationClassCharacteristic::OnRemoveCharacteristicButtonPressed()
{
	PopulationClass->Characteristics.RemoveAt(CharactericticIndex);
	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


// Contraint widget -------------------------

void SPopulationClassConstraint::Construct(const FArguments& InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	DetailBuilderPtr = InArgs._DetailBuilderPtr;
	CharactericticIndex = InArgs._CharactericticIndex;
	ConstraintIndex = InArgs._ConstraintIndex;
	ParameterIndex = InArgs._ParameterIndex;

	// Building a widget for each type of contraint
	BuildBoolWidget();
	BuildDiscreteWidget();
	BuildTagWidget();
	BuildRangeWidget();
	BuildCurveWidget();
	BuildDiscreteColorWidget();

	if (ParameterIndex > -1)
	{
		const bool multiConstraint = PopulationClass->Characteristics[CharactericticIndex].Constraints.Num() > 1;
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.0f, 3.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ConstraintType", "Constraint Type: "))
							.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
							.Visibility(multiConstraint ? EVisibility::Collapsed : EVisibility::Visible)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.0f, 3.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(FText::FromString(FString::Printf(TEXT("Constraint %d - Type: "), ConstraintIndex + 1)))
							.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
							.Visibility(multiConstraint ? EVisibility::Visible : EVisibility::Collapsed)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SAssignNew(ConstraintTypesComboBox, SComboBox<TSharedPtr<EPopulationConstraintType>>)
							.OptionsSource(&ConstraintTypes)
							.OnGenerateWidget(this, &SPopulationClassConstraint::OnContraintTypeGenerateWidget)
							.OnSelectionChanged(this, &SPopulationClassConstraint::OnComboBoxSelectionChanged)
							[
								SNew(STextBlock)
								.Text(this, &SPopulationClassConstraint::ComboBoxSelectionLabel)
							]
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.0f, 3.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ConstraintWeight", " - Weight: "))
							.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
							.Visibility(multiConstraint ? EVisibility::Visible : EVisibility::Collapsed)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SSpinBox<int32>)
							.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].ConstraintWeight; })
							.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].ConstraintWeight = InValue; PopulationClass->MarkPackageDirty(); })
							.OnValueChanged_Lambda([this](int32 InValue) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].ConstraintWeight = InValue; PopulationClass->MarkPackageDirty(); })
							.MinValue(1)
							.MaxValue(100)
							.ClearKeyboardFocusOnCommit(true)
							.MinDesiredWidth(30.0f)
							.Visibility(multiConstraint ? EVisibility::Visible : EVisibility::Collapsed)
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(10.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SButton)
							.ToolTipText(LOCTEXT("RemoveConstraint", "Remove Constraint"))
							.OnClicked(this, &SPopulationClassConstraint::OnRemoveConstraintButtonPressed)
							.Visibility(multiConstraint ? EVisibility::Visible : EVisibility::Collapsed)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush(TEXT("Cross")))
							]
						]
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					BoolWidget.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					DiscreteWidget.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					TagWidget.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					RangeWidget.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					CurveWidget.ToSharedRef()
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 5.0f, 0.0f, 5.0f)
				[
					DiscreteColorWidget.ToSharedRef()
				]
		];

		FillConstraintTypesOptions();
		FillConstraintTypesStringOptions();
		SetVisibilityCustom();
	}
}


void SPopulationClassConstraint::FillConstraintTypesOptions()
{
	ConstraintTypes.Empty();
	TArray<EPopulationConstraintType> Types = GetConstrinatTypes();
	for (EPopulationConstraintType Type : Types)
	{
		ConstraintTypes.Add(MakeShareable(new EPopulationConstraintType(Type)));
		if (PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Type == Type)
		{
			ConstraintTypesComboBox->SetSelectedItem(ConstraintTypes.Last());
		}
	}
}


TArray<EPopulationConstraintType> SPopulationClassConstraint::GetConstrinatTypes()
{
	switch (PopulationClass->CustomizableObject->GetPrivate()->GetParameterType(ParameterIndex))
	{
	case EMutableParameterType::None:
		break;
	case EMutableParameterType::Bool:
		return { EPopulationConstraintType::BOOL, EPopulationConstraintType::TAG };
	case EMutableParameterType::Int:
		return { EPopulationConstraintType::DISCRETE, EPopulationConstraintType::TAG };
	case EMutableParameterType::Float:
		return { EPopulationConstraintType::DISCRETE_FLOAT, EPopulationConstraintType::RANGE, EPopulationConstraintType::CURVE};
	case EMutableParameterType::Color:
		return { EPopulationConstraintType::CURVE_COLOR, EPopulationConstraintType::DISCRETE_COLOR };
	case EMutableParameterType::Projector:
		// Not implemented
	case EMutableParameterType::Texture:
		// Not implemented
	default:
		break;
	}
	
	return {};
}


void SPopulationClassConstraint::FillConstraintTypesStringOptions()
{
	ConstraintTypesStrings.Empty();
	ConstraintTypesStrings.Add(EPopulationConstraintType::NONE, "None");
	ConstraintTypesStrings.Add(EPopulationConstraintType::BOOL, "Bool");
	ConstraintTypesStrings.Add(EPopulationConstraintType::DISCRETE, "Discrete");
	ConstraintTypesStrings.Add(EPopulationConstraintType::DISCRETE_FLOAT, "Discrete");
	ConstraintTypesStrings.Add(EPopulationConstraintType::DISCRETE_COLOR, "Discrete");
	ConstraintTypesStrings.Add(EPopulationConstraintType::TAG, "Tag");
	ConstraintTypesStrings.Add(EPopulationConstraintType::RANGE, "Range");
	ConstraintTypesStrings.Add(EPopulationConstraintType::CURVE, "Curve");
	ConstraintTypesStrings.Add(EPopulationConstraintType::CURVE_COLOR, "Curve");
}


TSharedRef<SWidget> SPopulationClassConstraint::OnContraintTypeGenerateWidget(const TSharedPtr<EPopulationConstraintType> InMode) const
{
	return SNew(STextBlock).Text(FText::FromString(ConstraintTypesStrings[*InMode]));
}


void SPopulationClassConstraint::OnComboBoxSelectionChanged(const TSharedPtr<EPopulationConstraintType> InSelectedMode, ESelectInfo::Type SelectInfo)
{
	if (InSelectedMode.IsValid())
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Type = *InSelectedMode;
		SetVisibilityCustom();

		if (SelectInfo == ESelectInfo::OnMouseClick)
		{
			DetailBuilderPtr->ForceRefreshDetails();
			PopulationClass->MarkPackageDirty();
		}
	}
}


FText SPopulationClassConstraint::ComboBoxSelectionLabel() const
{
	if (ConstraintTypesComboBox->GetSelectedItem().IsValid())
	{
		return FText::FromString(ConstraintTypesStrings[*ConstraintTypesComboBox->GetSelectedItem()]);
	}

	return FText();
}


FReply SPopulationClassConstraint::OnRemoveConstraintButtonPressed()
{
	PopulationClass->Characteristics[CharactericticIndex].Constraints.RemoveAt(ConstraintIndex);
	PopulationClass->MarkPackageDirty();
	DetailBuilderPtr->ForceRefreshDetails();

	return FReply::Handled();
}


void SPopulationClassConstraint::BuildBoolWidget()
{
	BoolWidget = SNew(SVerticalBox);
	BoolWidget->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BoolTrueWeight", "True Weight:"))
			.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SSpinBox<int32>)
			.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TrueWeight; })
			.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) 
				{ 
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TrueWeight = InValue;
					PopulationClass->MarkPackageDirty();
				})
			.OnValueChanged_Lambda([this](int32 InValue)
				{
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TrueWeight = InValue;
					PopulationClass->MarkPackageDirty(); 
				})
			.MinValue(0)
			.MaxValue(100)
			.ClearKeyboardFocusOnCommit(true)
			.MinDesiredWidth(50.0f)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BoolFalseWeight", "False Weight:"))
			.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SSpinBox<int32>)
			.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].FalseWeight; })
			.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) 
				{ 
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].FalseWeight = InValue;
					PopulationClass->MarkPackageDirty(); 
				})
			.OnValueChanged_Lambda([this](int32 InValue)
				{
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].FalseWeight = InValue; 
					PopulationClass->MarkPackageDirty(); 
				})
			.MinValue(0)
			.MaxValue(100)
			.ClearKeyboardFocusOnCommit(true)
			.MinDesiredWidth(50.0f)
		]
	];
}


void SPopulationClassConstraint::BuildDiscreteWidget()
{
	DiscreteWidget = SNew(SVerticalBox);

	// TODO: Uncomment this slot when the discrete weight begin useful again
	//DiscreteWidget->AddSlot()
	//.AutoHeight()
	//[
	//	SNew(SHorizontalBox)
	//	+ SHorizontalBox::Slot()
	//	.AutoWidth()
	//	.Padding(0.0f, 3.0f, 0.0f, 0.0f)
	//	[
	//		SNew(STextBlock)
	//		.Text(LOCTEXT("DiscreteWeight", "Discrete Weight:"))
	//		.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
	//	]
	//	
	//	+ SHorizontalBox::Slot()
	//	.AutoWidth()
	//	.Padding(5.0f, 0.0f, 0.0f, 0.0f)
	//	[
	//		SNew(SSpinBox<int32>)
	//		.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteWeight; })
	//		.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) 
	//			{ 
	//				PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteWeight = InValue;
	//				PopulationClass->MarkPackageDirty();
	//			})
	//		.OnValueChanged_Lambda([this](int32 InValue) 
	//			{
	//				PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteWeight = InValue;
	//				PopulationClass->MarkPackageDirty();
	//			})
	//		.MinValue(1)
	//		.MaxValue(100)
	//		.ClearKeyboardFocusOnCommit(true)
	//		.MinDesiredWidth(50.0f)
	//	]
	//];
	
	// Filling combobox options
	if (ParameterIndex > -1 && PopulationClass->CustomizableObject->GetPrivate()->GetEnumParameterNumValues(ParameterIndex) > 0)
	{
		ParameterOptions.Empty();

		DiscreteWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f,3.0f,0.0f,0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DiscreteValue", "Discrete Value"))
				.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SAssignNew(ParameterOptionsComboBox, STextComboBox)
				.OptionsSource(&ParameterOptions)
				.OnSelectionChanged(this,&SPopulationClassConstraint::OnParameterOptionComboBoxSelectionChanged)
			]
		];

		for (int i = 0; i < PopulationClass->CustomizableObject->GetPrivate()->GetEnumParameterNumValues(ParameterIndex); ++i)
		{
			FString ParameterOption = PopulationClass->CustomizableObject->GetPrivate()->GetIntParameterAvailableOption(ParameterIndex, i);

			ParameterOptions.Add(MakeShareable(new FString(ParameterOption)));
			if (ParameterOption == PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteValue)
			{
				ParameterOptionsComboBox->SetSelectedItem(ParameterOptions.Last());
			}
		}
	}
}


void SPopulationClassConstraint::OnParameterOptionComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid())
	{
		ParameterOptionsComboBox->SetSelectedItem(Selection);
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteValue = *Selection;

		if (SelectInfo == ESelectInfo::OnMouseClick)
		{
			PopulationClass->MarkPackageDirty();
		}
	}
}


void SPopulationClassConstraint::BuildTagWidget()
{
	TagWidget = SNew(SVerticalBox);

	// TODO: delete this slot when the tags begin useful
	TagWidget->AddSlot()
	.AutoHeight()
	[
		SNew(SPopulationClassTagManager)
		.PopulationClass(PopulationClass)
		.DetailBuilderPtr(DetailBuilderPtr)
		.AllowlistPtr(&PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Allowlist)
		.BlocklistPtr(&PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Blocklist)
	];

	// TODO: Uncomment this slot when the tags begin useful
	/*TagWidget->AddSlot()
	.AutoHeight()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ParameterOptionTagWeight","Tag Weight: "))
				.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<int32>)
				.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TagWeight; })
				.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType)
				{
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TagWeight = InValue;
					PopulationClass->MarkPackageDirty();
				})
				.OnValueChanged_Lambda([this](int32 InValue)
				{
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].TagWeight = InValue;
					PopulationClass->MarkPackageDirty();
				})
				.MinValue(1)
				.MaxValue(100)
				.ClearKeyboardFocusOnCommit(true)
				.MinDesiredWidth(50.0f)
			]
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SPopulationClassTagManager)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.AllowlistPtr(&PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Allowlist)
			.BlocklistPtr(&PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Blocklist)
		]
	];*/
}


void SPopulationClassConstraint::BuildRangeWidget()
{
	RangeWidget = SNew(SVerticalBox);

	bool bDiscrete = PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Type == EPopulationConstraintType::DISCRETE_FLOAT;

	// Adding a Texture if the parameter option has one
	if (ParameterIndex >=0 && PopulationClass->CustomizableObject && PopulationClass->CustomizableObjectInstance)
	{
		RangeWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 10.0f, 0.0f, 5.0f)
		[
			SNew(SRangeSquare)
			.Ranges(&PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges)
			.bDiscrete(bDiscrete)
		];
	}
	
	if (!bDiscrete)
	{
		RangeWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("AddRangeButtonText", "Add Range"))
				.OnClicked(this, &SPopulationClassConstraint::OnAddRangeButtonPressed)
			]
		];

		for (int32 i = 0; i < PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Num(); ++i)
		{
			RangeWidget->AddSlot()
			.AutoHeight()
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SNew(SRangeWidget)
				.PopulationClass(PopulationClass)
				.DetailBuilderPtr(DetailBuilderPtr)
				.CharactericticIndex(CharactericticIndex)
				.ConstraintIndex(ConstraintIndex)
				.RangeIndex(i)
			];
		}
	}
	
	else if(PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Type == EPopulationConstraintType::DISCRETE_FLOAT)
	{
		RangeWidget->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SRangeWidget)
			.PopulationClass(PopulationClass)
			.DetailBuilderPtr(DetailBuilderPtr)
			.CharactericticIndex(CharactericticIndex)
			.ConstraintIndex(ConstraintIndex)
			.RangeIndex(0)
			.bDiscrete(true)
		];
	}
}


void SPopulationClassConstraint::BuildCurveWidget()
{
	CurveWidget = SNew(SVerticalBox);
	CurveWidget->AddSlot()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("CurveAssetText", "Curve Asset:"))
			.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UCurveBase::StaticClass())
			.OnObjectChanged(this, &SPopulationClassConstraint::OnSelectCurveAsset)
			.ObjectPath(PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Curve->GetPathName())
			.ForceVolatile(true)
			.DisplayThumbnail(true)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.Text(LOCTEXT("ShowCurveAsset", "Open in Editor"))
			.OnClicked_Lambda([this]()
			{
				if (PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Curve)
				{
					PopulationClass->EditorCurve = PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Curve;
				}

				return FReply::Handled();
			}
			)
		]
	];


	if (ConstraintTypesComboBox.IsValid() && ConstraintTypesComboBox->GetSelectedItem().IsValid())
	{
		EPopulationConstraintType type = *ConstraintTypesComboBox->GetSelectedItem();
		UCurveLinearColor* Curve = Cast<UCurveLinearColor>(PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Curve);
		if(Curve && type == EPopulationConstraintType::CURVE)
		{
			ColorOptions.Empty();
			ColorOptions.Add(MakeShareable(new FString("R")));
			ColorOptions.Add(MakeShareable(new FString("G")));
			ColorOptions.Add(MakeShareable(new FString("B")));
			ColorOptions.Add(MakeShareable(new FString("A")));

			TSharedPtr<FString> SelectedCurve;

			int32 ColorIndex = (int32)PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].CurveColor;
			SelectedCurve = ColorOptions[ColorIndex];

			CurveWidget->AddSlot()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 3.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectedCurveText", "Selected Curve:"))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(ColorOptionsComboBox,STextComboBox)
					.OptionsSource(&ColorOptions)
					.InitiallySelectedItem(SelectedCurve)
					.OnSelectionChanged(this, &SPopulationClassConstraint::OnColorCurveSelectionChanged)
				]
			];
		}
	}
}

void SPopulationClassConstraint::OnColorCurveSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	ColorOptionsComboBox->SetSelectedItem(Selection);

	if (*Selection == "R")
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].CurveColor = ECurveColor::RED;
	}
	else if (*Selection == "G")
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].CurveColor = ECurveColor::GREEN;
	}
	else if(*Selection == "B")
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].CurveColor = ECurveColor::BLUE;
	}
	else
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].CurveColor = ECurveColor::ALPHA;
	}
	PopulationClass->MarkPackageDirty();
}

void SPopulationClassConstraint::BuildDiscreteColorWidget()
{
	DiscreteColorWidget = SNew(SVerticalBox);
	DiscreteColorWidget->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 10.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("DiscreteColorText", "Color constant:"))
			.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SColorBlock)
			.Color_Lambda([&]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteColor; })
			.ShowBackgroundForAlpha(true)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.Size(FVector2D(50.0f, 20.0f))
			.OnMouseButtonDown(this, &SPopulationClassConstraint::OnColorPreviewClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SColorBlock)
			.Color_Lambda([&]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteColor; })
			.ShowBackgroundForAlpha(true)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.Size(FVector2D(50.0f, 20.0f))
			.OnMouseButtonDown(this, &SPopulationClassConstraint::OnColorPreviewClicked)
		]
	];
}

FReply SPopulationClassConstraint::OnColorPreviewClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	FColorPickerArgs args;
	args.bIsModal = true;
	args.bUseAlpha = true;
	args.bOnlyRefreshOnMouseUp = false;
	args.InitialColor = PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteColor;
	args.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SPopulationClassConstraint::OnSetConstantColorFromColorPicker);

	OpenColorPicker(args);

	return FReply::Handled();
}

void SPopulationClassConstraint::OnSetConstantColorFromColorPicker(FLinearColor InColor)
{
	PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].DiscreteColor = InColor;
	PopulationClass->MarkPackageDirty();
}

void SPopulationClassConstraint::SetVisibilityCustom()
{
	BoolWidget->SetVisibility(EVisibility::Collapsed);
	DiscreteWidget->SetVisibility(EVisibility::Collapsed);
	TagWidget->SetVisibility(EVisibility::Collapsed);
	RangeWidget->SetVisibility(EVisibility::Collapsed);
	CurveWidget->SetVisibility(EVisibility::Collapsed);
	DiscreteColorWidget->SetVisibility(EVisibility::Collapsed);

	if (ConstraintTypesComboBox.IsValid() && ConstraintTypesComboBox->GetSelectedItem().IsValid())
	{
		EPopulationConstraintType type = *ConstraintTypesComboBox->GetSelectedItem();

		switch (type)
		{
		case EPopulationConstraintType::NONE:
			break;
		case EPopulationConstraintType::BOOL:
			BoolWidget->SetVisibility(EVisibility::Visible);
			break;
		case EPopulationConstraintType::DISCRETE:
			DiscreteWidget->SetVisibility(EVisibility::Visible);
			break;
		case EPopulationConstraintType::TAG:
			TagWidget->SetVisibility(EVisibility::Visible);
			break;
		case EPopulationConstraintType::DISCRETE_FLOAT:
		case EPopulationConstraintType::RANGE:
			RangeWidget->SetVisibility(EVisibility::Visible);
			break;
		case EPopulationConstraintType::CURVE:
		case EPopulationConstraintType::CURVE_COLOR:
			CurveWidget->SetVisibility(EVisibility::Visible);
			break;
		case EPopulationConstraintType::DISCRETE_COLOR:
			DiscreteColorWidget->SetVisibility(EVisibility::Visible);
			break;
		default:
			break;
		}
	}
}


void SPopulationClassConstraint::OnSelectCurveAsset(const FAssetData & AssetData) 
{
	if (PopulationClass)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Curve = Cast<UCurveBase>(AssetData.GetAsset());
		PopulationClass->MarkPackageDirty();
		DetailBuilderPtr->ForceRefreshDetails();
	}
}


FReply SPopulationClassConstraint::OnAddRangeButtonPressed()
{
	if (PopulationClass && DetailBuilderPtr)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Add(FConstraintRanges());
		PopulationClass->MarkPackageDirty();
		DetailBuilderPtr->ForceRefreshDetails();
	}

	return FReply::Handled();
}


// Range Widget -------------------------

void SRangeWidget::Construct(const FArguments& InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	DetailBuilderPtr = InArgs._DetailBuilderPtr;
	CharactericticIndex = InArgs._CharactericticIndex;
	ConstraintIndex = InArgs._ConstraintIndex;
	RangeIndex = InArgs._RangeIndex;
	bDiscrete = InArgs._bDiscrete;

	int32 TotalNumRanges = PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Num();

	if (!bDiscrete)
	{
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FString::Printf(TEXT("Range %d - "), RangeIndex)))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MinText", "Min: "))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SSpinBox<float>)
					.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue; })
					.OnValueCommitted(this, &SRangeWidget::OnMinValueCommited)
					.OnValueChanged(this, &SRangeWidget::OnMinValueChanged)
					.MinValue(0.0f)
					.MaxValue(1.0f)
					.ClearKeyboardFocusOnCommit(true)
					.MinDesiredWidth(30.0f)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MaxText", "Max: "))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(3.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SSpinBox<float>)
					.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue; })
					.OnValueCommitted(this, &SRangeWidget::OnMaxValueCommited)
					.OnValueChanged(this, &SRangeWidget::OnMaxValueChanged)
					.MinValue(0.0f)
					.MaxValue(1.0f)
					.ClearKeyboardFocusOnCommit(true)
					.MinDesiredWidth(30.0f)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(15.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ToolTipText(LOCTEXT("RemoveRange", "Remove Range"))
					.OnClicked(this, &SRangeWidget::OnRemoveRangeButtonPressed)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Cross")))
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RangeWeightText", "Range Weight:"))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
					.Visibility(TotalNumRanges > 1 ? EVisibility::Visible : EVisibility::Collapsed)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SSpinBox<int32>)
					.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight; })
					.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight = InValue; PopulationClass->MarkPackageDirty(); })
					.OnValueChanged_Lambda([this](int32 InValue) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight = InValue; PopulationClass->MarkPackageDirty(); })
					.MinValue(1)
					.MaxValue(100)
					.ClearKeyboardFocusOnCommit(true)
					.MinDesiredWidth(30.0f)
					.Visibility(TotalNumRanges > 1 ? EVisibility::Visible : EVisibility::Collapsed)
				]
			]
		];
	}
	else
	{
		if (TotalNumRanges > 0)
		{
			if (TotalNumRanges > 1)
			{
				// if there is more than one we delete all the values and create a new one
				PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Empty();
				PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Add(FConstraintRanges());
			}
			else
			{
				// If we pass from range to discrete the value of the maximum will be equal to the value of the minimum
				PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[0].MaximumValue =
					PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[0].MinimumValue;
			}
		}
		else
		{
			// Discrete floats do not have a button to add a value, we have to add it here
			PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.Add(FConstraintRanges());
		}

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FloatValue", "Discrete Value:"))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SSpinBox<float>)
					.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue; })
					.OnValueCommitted(this, &SRangeWidget::OnMinValueCommited)
					.OnValueChanged(this, &SRangeWidget::OnMinValueChanged)
					.MinValue(0.0f)
					.MaxValue(1.0f)
					.ClearKeyboardFocusOnCommit(true)
					.MinDesiredWidth(50.0f)
				]
			]

			// TODO: Uncomment this slot when this begins useful
			/*+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DiscreteFloatWeightText", "Discrete Weight:"))
					.ColorAndOpacity(CharactericticIndex % 2 == 0 ? EvenTextColor : OddTextColor)
					.Visibility(TotalNumRanges > 1 ? EVisibility::Visible : EVisibility::Collapsed)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SSpinBox<int32>)
					.Value_Lambda([this]() { return PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight; })
					.OnValueCommitted_Lambda([this](int32 InValue, ETextCommit::Type InCommitType) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight = InValue; PopulationClass->MarkPackageDirty(); })
					.OnValueChanged_Lambda([this](int32 InValue) { PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].RangeWeight = InValue; PopulationClass->MarkPackageDirty(); })
					.MinValue(1)
					.MaxValue(100)
					.ClearKeyboardFocusOnCommit(true)
					.MinDesiredWidth(30.0f)
					.Visibility(TotalNumRanges > 1 ? EVisibility::Visible : EVisibility::Collapsed)
				]
			]*/
		];
	}
}


FReply SRangeWidget::OnRemoveRangeButtonPressed()
{
	if (PopulationClass && DetailBuilderPtr)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges.RemoveAt(RangeIndex);
		PopulationClass->MarkPackageDirty();
		DetailBuilderPtr->ForceRefreshDetails();
	}

	return FReply::Handled();
}


void SRangeWidget::OnMinValueCommited(float InValue, ETextCommit::Type InCommitType)
{
	if(bDiscrete)
	{ 
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue 
			= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue = InValue;
	}
	else
	{
		if (InValue <= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue)
		{
			PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue = InValue;
		}
	}

	PopulationClass->MarkPackageDirty();
}


void SRangeWidget::OnMaxValueCommited(float InValue, ETextCommit::Type InCommitType)
{
	if (InValue >= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue = InValue;
		PopulationClass->MarkPackageDirty();
	}
}
	

void SRangeWidget::OnMinValueChanged(float InValue) 
{
	if (bDiscrete)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue
			= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue = InValue;
	}
	else
	{
		if (InValue <= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue)
		{
			PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue = InValue;
		}
	}

	PopulationClass->MarkPackageDirty();
}


void SRangeWidget::OnMaxValueChanged(float InValue)
{
	if (InValue >= PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MinimumValue)
	{
		PopulationClass->Characteristics[CharactericticIndex].Constraints[ConstraintIndex].Ranges[RangeIndex].MaximumValue = InValue;
		PopulationClass->MarkPackageDirty();
	}
}


// Custom Editor Line -------------------------

void SCustomEditorLine::Construct(const FArguments& InArgs)
{
	LineColor = InArgs._LineColor;
	Length = InArgs._Length;
	Horizontal = InArgs._Horizontal;
}


int32 SCustomEditorLine::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, 
	const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 RetLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const FVector2D BorderPadding = FVector2D(2, 2);
	float LineLength;

	if (Length > 0.0f)
	{
		// Using the specified length
		LineLength = Length;
	}
	else
	{
		// If the length is 0 then we use the size of the container
		if (Horizontal)
		{
			LineLength = AllottedGeometry.Size.X - 2.0f * BorderPadding.X;
		}
		else
		{
			LineLength = AllottedGeometry.Size.Y - 2.0f * BorderPadding.X;
		}
	}

	TArray< FVector2D > LinePoints;
	LinePoints.SetNum(2);

	LinePoints[0] = FVector2D(BorderPadding.X , BorderPadding.Y);
	if (Horizontal)
	{
		LinePoints[1] = FVector2D(BorderPadding.X + LineLength, BorderPadding.Y);
	}
	else
	{
		LinePoints[1] = FVector2D(BorderPadding.X, BorderPadding.Y + LineLength);
	}
	FSlateDrawElement::MakeLines(OutDrawElements, ++RetLayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, LineColor, false, 2.0);

	return RetLayerId;
}


// Range Square Widget -------------------------

void SRangeSquare::Construct(const FArguments& InArgs)
{
	SquareColor	= InArgs._SquareColor;
	Ranges = InArgs._Ranges;
	bDiscrete = InArgs._bDiscrete;
	
	Brush.TintColor = FSlateColor(FLinearColor::Gray);
	Brush.ImageSize.X = 400.0f;
	Brush.ImageSize.Y = 10.0f;
	Brush.DrawAs = ESlateBrushDrawType::Image;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SImage)
			.Image(&Brush)
		]
	];
	
	bMouseDownMin = bMouseDownMax = false;
	TextureRectangle = Brush.ImageSize;
}


int32 SRangeSquare::OnPaint(const FPaintArgs & Args, const FGeometry & AllottedGeometry, const FSlateRect & MyClippingRect, FSlateWindowElementList & OutDrawElements, int32 LayerId, const FWidgetStyle & InWidgetStyle, bool bParentEnabled) const
{
	int32 RetLayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyClippingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Function to draw the squares
	const auto MakeSquareLine = [&](const TArray<FVector2D>& Points)
	{
		FSlateDrawElement::MakeLines
		(
			OutDrawElements, ++RetLayerId, AllottedGeometry.ToPaintGeometry(), Points,
			ESlateDrawEffect::None,	SquareColor, true,2.0
		);
	};

	float TopMargin = 3.0f;

	TArray<FVector2D> SquareLinePoints;
	SquareLinePoints.SetNum(4);

	// Square edges
	for (int32 r = 0; r < Ranges->Num(); ++r)
	{
		// BottomLeft
		SquareLinePoints[0] = FVector2D(TextureRectangle.X*(*Ranges)[r].MaximumValue, TextureRectangle.Y + TopMargin);
		// BottomRight
		SquareLinePoints[1] = FVector2D(TextureRectangle.X*(*Ranges)[r].MinimumValue, TextureRectangle.Y + TopMargin);
		// TopRight
		SquareLinePoints[2] = FVector2D(TextureRectangle.X*(*Ranges)[r].MinimumValue, -TopMargin );
		// TopLeft
		SquareLinePoints[3] = FVector2D(TextureRectangle.X*(*Ranges)[r].MaximumValue, -TopMargin );

		for (int32 i = 0; i < SquareLinePoints.Num(); ++i)
		{
			TArray<FVector2D> Line;
			Line.Add(SquareLinePoints[i]);
			Line.Add(SquareLinePoints[(i + 1) % 4]);

			MakeSquareLine(Line);
		}
	}

	return RetLayerId; 
}


FReply SRangeSquare::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	float SelectionOffset = 0.015f;

	SelectedRangeIndex = -1;

	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (bDiscrete)
		{
			if (Ranges->Num() > 0)
			{
				bMouseDownMax = true;
				SelectedRangeIndex = 0;
			}
		}
		else
		{
			float MousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X;
			float MouseToTexture = MousePos / TextureRectangle.X;

			for (int32 i = Ranges->Num()-1; i >= 0; --i)
			{
				if (abs(MouseToTexture - (*Ranges)[i].MaximumValue) <= SelectionOffset)
				{
					bMouseDownMax = true;
					SelectedRangeIndex = i;

					break;
				}

				if (abs(MouseToTexture - (*Ranges)[i].MinimumValue) <= SelectionOffset)
				{
					bMouseDownMin = true;
					SelectedRangeIndex = i;

					break;
				}
			}
		}
	}
	return SCompoundWidget::OnMouseButtonDown(MyGeometry, MouseEvent);
}


FReply SRangeSquare::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
   	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bMouseDownMin = bMouseDownMax = false;
	}
	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}


FReply SRangeSquare::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	float MousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X;

	if ((bMouseDownMin || bMouseDownMax) && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		if (SelectedRangeIndex != -1 && MousePos <= TextureRectangle.X)
		{
			float NewValue = MousePos / TextureRectangle.X;

			if (bDiscrete)
			{
				(*Ranges)[SelectedRangeIndex].MinimumValue = (*Ranges)[SelectedRangeIndex].MaximumValue = NewValue;
			}
			
			if (bMouseDownMax && NewValue >= (*Ranges)[SelectedRangeIndex].MinimumValue)
			{
				(*Ranges)[SelectedRangeIndex].MaximumValue = NewValue;
			}

			if (bMouseDownMin && NewValue <= (*Ranges)[SelectedRangeIndex].MaximumValue)
			{
				(*Ranges)[SelectedRangeIndex].MinimumValue = NewValue;
			}
		}
	}

	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}


void SRangeSquare::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	bMouseDownMin = bMouseDownMax = false;
}


#undef LOCTEXT_NAMESPACE
