// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Styling/SlateBrush.h"
#include "Widgets/SCompoundWidget.h"

enum class EPopulationConstraintType : uint8;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
struct FConstraintRanges;
template <typename OptionType> class SComboBox;

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class IDetailLayoutBuilder;
class STextComboBox;
class SVerticalBox;
class SWidget;
class UCustomizableObjectPopulationClass;
class UTexture;
struct FAssetData;
struct FGeometry;
struct FPointerEvent;

// Widgt that draws a Square to represent a range
class SRangeSquare : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRangeSquare) : _SquareColor(FColor(255, 180, 180, 255)),_bDiscrete(false){}
	SLATE_ARGUMENT(FColor, SquareColor)
	SLATE_ARGUMENT(TArray<FConstraintRanges>*, Ranges)
	SLATE_ARGUMENT(bool, bDiscrete)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	
	/** Allow mouse management */
	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void OnMouseLeave(const FPointerEvent& MouseEvent) override;

private:

	/** Color of the Square */
	FColor SquareColor;

	/** Pointer to the list of ranges */
	TArray<FConstraintRanges>* Ranges;

	int32 SelectedRangeIndex;

	/** Size of the texture decoration of the parameter */
	FVector2D TextureRectangle;

	/** The range will be a single value */
	bool bDiscrete;

	/** Indicates if the mouse is down on the Min Value */
	bool bMouseDownMin;

	/** Indicates if the mouse is down on the Max*/
	bool bMouseDownMax;

	/** Brush to draw the decorations of a parameter option */
	FSlateBrush Brush;
};


//Widget to manage the ranges of a constraint
class SRangeWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SRangeWidget):_bDiscrete(false){}
	SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
	SLATE_ARGUMENT(IDetailLayoutBuilder*, DetailBuilderPtr)
	SLATE_ARGUMENT(int32, CharactericticIndex)
	SLATE_ARGUMENT(int32, ConstraintIndex)
	SLATE_ARGUMENT(int32, RangeIndex)
	SLATE_ARGUMENT(bool, bDiscrete)
	SLATE_ARGUMENT(bool, bRanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback to remove a Range from the Ranges list of the contraint */
	FReply OnRemoveRangeButtonPressed();
	
	/** Callbacks for the commit of the value of the SpinBox */
	void OnMinValueCommited(float InValue, ETextCommit::Type InCommitType);
	void OnMaxValueCommited(float InValue, ETextCommit::Type InCommitType);
	
	/** Callbacks for the change of the value of the SpinBox */
	void OnMinValueChanged(float InValue);
	void OnMaxValueChanged(float InValue);

private:

	/** Pointer to the Population class of the Detials view */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** Pointer to the builder passed by CustomizeDetails method */
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** Index of the characteristic */
	int32 CharactericticIndex;

	/** Index of the contraint */
	int32 ConstraintIndex;

	/** Index of the Range */
	int32 RangeIndex;

	/** The range will be a single value */
	bool bDiscrete;

};


// Widget to draw a line in the editor
class SCustomEditorLine : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SCustomEditorLine) : _LineColor(FColor(50,50,50,255)), _Length(0.0f),_Horizontal(true) {}
		SLATE_ARGUMENT(FColor, LineColor)
		SLATE_ARGUMENT(float, Length)
		SLATE_ARGUMENT(bool, Horizontal)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:

	/** Color of the line */
	FColor LineColor;

	/** Length of the line, if 0.0 then will be the length of the tab */
	float Length;

	/** Bool that represents de orientation of the line. True means an horizontal line, false means a vertical line */
	bool Horizontal;
};


// Widget to represent a single constraint of a Characteristic
class SPopulationClassConstraint : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPopulationClassConstraint) {}
	SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
	SLATE_ARGUMENT(IDetailLayoutBuilder*, DetailBuilderPtr)
	SLATE_ARGUMENT(int32, CharactericticIndex)
	SLATE_ARGUMENT(int32, ConstraintIndex)
	SLATE_ARGUMENT(int32, ParameterIndex)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Fills the combobox option source */
	void FillConstraintTypesOptions();
	
	/** Fills the contraint types option map to get the strings from the enums */
	void FillConstraintTypesStringOptions();

	/** Returns an array with all the contraint types */
	TArray<EPopulationConstraintType> GetConstrinatTypes();

	/** Builds the contraint types combobox widget */
	TSharedRef<SWidget> OnContraintTypeGenerateWidget(const TSharedPtr<EPopulationConstraintType> InMode) const;
	
	/** Callback when the contraint type changes */
	void OnComboBoxSelectionChanged(const TSharedPtr<EPopulationConstraintType> InSelectedMode, ESelectInfo::Type SelectInfo);
	
	/** Builds the label of the contraints ComboBox */
	FText ComboBoxSelectionLabel() const;

	/** Callback to remove the contraint */
	FReply OnRemoveConstraintButtonPressed();

	/** Callback for the selection of the curve asset */
	void OnSelectCurveAsset(const FAssetData & AssetData);

	/** Callback that adds a range to the range contraint */
	FReply OnAddRangeButtonPressed();

	/** Shows a different widget in function of the type of the constraint */
	void SetVisibilityCustom();

	/** Builders for the different types of contraint widgets */
	void BuildBoolWidget();
	void BuildDiscreteWidget();
	void BuildTagWidget();
	void BuildRangeWidget();
	void BuildCurveWidget();
	void BuildDiscreteColorWidget();

	/** Callbacks to select a parameter option */
	void OnParameterOptionComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	void OnColorCurveSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	FReply OnColorPreviewClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	void OnSetConstantColorFromColorPicker(FLinearColor InColor);

private:

	/** Pointer to the Population class of the Detials view */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** Pointer to the builder passed by CustomizeDetails method */
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** Index of the characteristic */
	int32 CharactericticIndex;
	
	/** Index of the contraint */
	int32 ConstraintIndex;

	/** Index of the parameter */
	int32 ParameterIndex;

	/** List of parameters of the customizable object for the parameters combobox */
	TArray< TSharedPtr<FString>> CustomizableObjectParameters;

	/** List with the types of constraints */
	TArray< TSharedPtr < EPopulationConstraintType > > ConstraintTypes;

	/** Map that relates the Constraint type enum with the string */
	TMap<EPopulationConstraintType, FString> ConstraintTypesStrings;

	/** ComboBox to selec the type of a constraint */
	TSharedPtr<SComboBox<TSharedPtr < EPopulationConstraintType > > > ConstraintTypesComboBox;

	/** Widgets for the different types of Constraints */
	TSharedPtr<SVerticalBox> BoolWidget;
	TSharedPtr<SVerticalBox> DiscreteWidget;
	TSharedPtr<SVerticalBox> TagWidget;
	TSharedPtr<SVerticalBox> RangeWidget;
	TSharedPtr<SVerticalBox> CurveWidget;
	TSharedPtr<SVerticalBox> DiscreteColorWidget;

	/** List with all the parameter options of the customizable object */
	TArray<TSharedPtr<FString>> ParameterOptions;

	/** ComboBox to select the parameter of a discrete contraint */
	TSharedPtr<STextComboBox> ParameterOptionsComboBox;

	/** Combobox to select the used channel of the color curve*/
	TSharedPtr<STextComboBox> ColorOptionsComboBox;

	/** Options of the Color Combobox R,G,B,A */
	TArray< TSharedPtr<FString> > ColorOptions;

	// Pointer to store the color editors to manage the visibility
	TSharedPtr<SVerticalBox> ColorEditor;
};


// Widget that manages a single characteristic of the population class
class SPopulationClassCharacteristic : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPopulationClassCharacteristic) {}
	SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
	SLATE_ARGUMENT(IDetailLayoutBuilder*, DetailBuilderPtr)
	SLATE_ARGUMENT(int32, CharactericticIndex)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback that changes the parameter of the characteristic */
	void OnParameterSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Callback for the button that adds a constraint to the characteristic */
	FReply OnAddConstraintButtonPessed();

	/** Callback for the button that removes the characteristic from the population class */
	FReply OnRemoveCharacteristicButtonPressed();

private:

	/** Pointer to the Population class of the Detials view */
	UCustomizableObjectPopulationClass* PopulationClass;
	
	/** Pointer to the builder passed by CustomizeDetails method */
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** Index of the Characteristic being edited */
	int32 CharactericticIndex;

	/** List of parameters of the customizable object for the parameters combobox */
	TArray< TSharedPtr<FString>> CustomizableObjectParameters;

	/** Widget to manage the constraints of a Characteristic of the Population Class */
	TSharedPtr<SVerticalBox> ConstraintsWidgets;
};


// Widget that manages a single list
class SPopulationClassTagList : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPopulationClassTagList) {}
		SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
		SLATE_ARGUMENT(IDetailLayoutBuilder*, DetailBuilderPtr)
		SLATE_ARGUMENT(int32, TagIndex)
		SLATE_ARGUMENT(TArray< FString >*, ListPtr)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Callback for the combobox selection changed */
	void OnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Callback for the button that removes a tag from a list */
	FReply OnRemoveTagButtonPressed(int32 Index);

private:

	/** Pointer to the Population class of the Detials view */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** String Value to print the tag */
	FString TagValue;
	
	/** Index of the tag inside its List */
	int32 TagIndex;

	/** Option Source for the tags combobox  */
	TArray< TSharedPtr<FString>> AllTags;

	/** Pointer to the list to modify in this widget */
	TArray< FString >* ListPtr;

	/** Pointer to the builder passed by CustomizeDetails method */
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** Combobox to select a tag */
	TSharedPtr<STextComboBox> TagsComboBox;

};


// Widget to manage the white and block lists of a Population Class
class SPopulationClassTagManager : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SPopulationClassTagManager) {}
	SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
	SLATE_ARGUMENT(IDetailLayoutBuilder*, DetailBuilderPtr)
	SLATE_ARGUMENT(TArray< FString >*, AllowlistPtr)
	SLATE_ARGUMENT(TArray< FString >*, BlocklistPtr)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback to add a new tag to on of the lists */
	FReply OnAddTagButtonPressed(bool bIsAllowlist);

	/** List Widget Builders */
	void BuildAllowlistWidget();
	void BuildBlocklistWidget();

private:

	/** Widgets to view and add tags of the lists */
	TSharedPtr<SVerticalBox> AllowlistWidget;
	TSharedPtr<SVerticalBox> BlocklistWidget;

	/** Pointer to the list to modify in this widget */
	TArray< FString >* AllowlistPtr;

	/** Pointer to the list to modify in this widget */
	TArray< FString >* BlocklistPtr;

	/** Pointer to the builder passed by CustomizeDetails method */
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

	/** Pointer to the Population class of the Detials view */
	UCustomizableObjectPopulationClass* PopulationClass;
};


// Details view of the Customizable Object Population Class UObject
class FCustomizableObjectPopulationClassDetails : public IDetailCustomization
{
public:

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** ILayoutDetails interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	/** Builds the Characteristics widget */
	void BuildCharacteristicsWidgets();

	/** Button callback that adds a Characteristic to the Population Class */
	FReply OnAddCharacteristicButtonPessed();

	/** Callback of the Customizable Object Property */
	void OnCustomizableObjectPropertyChanged();

private:

	/** Pointer to the Population Class open in the Editor */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** Widget to manageme the white and block lists of the Population Class */
	TSharedPtr<SPopulationClassTagManager> TagManagementWidgets;
	
	/** Widget to manage tje characteristics of the Population Class*/
	TSharedPtr<SVerticalBox> CharacteristicsWidgets;

	// Pointer to the builder passed by CustomizeDetails method
	IDetailLayoutBuilder* DetailBuilderPtr = nullptr;

};
