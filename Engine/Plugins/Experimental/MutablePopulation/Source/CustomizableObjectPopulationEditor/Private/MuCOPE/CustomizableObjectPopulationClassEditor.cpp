// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationClassEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ContentBrowserModule.h"
#include "Curves/CurveBase.h"
#include "DetailsViewArgs.h"
#include "Engine/SkeletalMesh.h"
#include "FileHelpers.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"
#include "MuCOP/CustomizableObjectPopulation.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"
#include "MuCOP/CustomizableObjectPopulationGenerator.h"
#include "MuCOPE/CustomizableObjectPopulationClassEditorActions.h"
#include "MuCOPE/CustomizableObjectPopulationEditorModule.h"
#include "MuCOPE/SCustomizableObjectPopulationClassTagsTool.h"
#include "MuCOPE/SCustomizableObjectPopulationEditorViewport.h"
#include "PropertyEditorModule.h"
#include "SCurveEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

class IToolkitHost;
class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClassEditor"

const FName FCustomizableObjectPopulationClassEditor::PopulationClassPropertiesTabId(TEXT("PopulationClassEditor_Properties"));
const FName FCustomizableObjectPopulationClassEditor::PopulationClassTagsToolId(TEXT("PopulationClassEditor_TagsTool"));
const FName FCustomizableObjectPopulationClassEditor::CurveEditorTabId(TEXT("PopulationClassCurveEditor_Tab"));
const FName FCustomizableObjectPopulationClassEditor::ViewportTabId(TEXT("PopulationClassViewport_Tab"));

void FCustomizableObjectPopulationClassEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	InTabManager->RegisterTabSpawner(PopulationClassPropertiesTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SpawnTab_PopulationClassProperties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Properties"))
		.SetGroup(MenuStructure.GetToolsCategory());

	InTabManager->RegisterTabSpawner(PopulationClassTagsToolId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SpawnTab_PopulationClassTagsTool))
		.SetDisplayName(LOCTEXT("TagsToolTab", "Tags Tool"))
		.SetGroup(MenuStructure.GetToolsCategory());

	InTabManager->RegisterTabSpawner(CurveEditorTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SpawnTab_CurveEditor))
		.SetDisplayName(LOCTEXT("CurveEditorTab", "Curve Editor"))
		.SetGroup(MenuStructure.GetToolsCategory());
	
	InTabManager->RegisterTabSpawner(ViewportTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("PopulationClassViewportTab", "Viewport"))
		.SetGroup(MenuStructure.GetToolsCategory());
}


void FCustomizableObjectPopulationClassEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(PopulationClassPropertiesTabId);
	InTabManager->UnregisterTabSpawner(PopulationClassTagsToolId);
	InTabManager->UnregisterTabSpawner(CurveEditorTabId);
	InTabManager->UnregisterTabSpawner(ViewportTabId);
}


FCustomizableObjectPopulationClassEditor::FCustomizableObjectPopulationClassEditor()
{
	PopulationClass = nullptr;
	CurrentEditorCurve = nullptr;
}


FCustomizableObjectPopulationClassEditor::~FCustomizableObjectPopulationClassEditor()
{
	PopulationClass->CustomizableObjectInstance = nullptr;
	PopulationClass = nullptr;
	CurrentEditorCurve = nullptr;

	PreviewCustomizableSkeletalComponents.Empty();
	PreviewSkeletalMeshComponents.Empty();
	ViewportInstances.Empty();

	/*for (int32 i = 0; i < PreviewCustomizableSkeletalComponents.Num(); ++i)
	{
		PreviewCustomizableSkeletalComponents[i] = nullptr;
	}

	for (int32 i = 0; i < PreviewSkeletalMeshComponents.Num(); ++i)
	{
		PreviewSkeletalMeshComponents[i] = nullptr;
	}

	for (int32 i = 0; i < ViewportInstances.Num(); ++i)
	{
		ViewportInstances[i] = nullptr;
	}*/
}


void FCustomizableObjectPopulationClassEditor::InitCustomizableObjectPopulationClassEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCustomizableObjectPopulationClass* InObject)
{
	PopulationClass = InObject;

	FCustomizableObjectPopulationClassEditorCommands::Register();

	BindCommands();
	ExpandToolBar();

	if (PopulationClass->CustomizableObject)
	{
		if (PopulationClass->CustomizableObject->IsCompiled() && !PopulationClass->CustomizableObjectInstance)
		{
			PopulationClass->CustomizableObjectInstance = PopulationClass->CustomizableObject->CreateInstance();
			PopulationClass->CustomizableObjectInstance->SetBuildParameterRelevancy(true);
			PopulationClass->CustomizableObjectInstance->UpdateSkeletalMeshAsync(true, true);

			bRefreshDetailsView = true;
		}
	}

	// Population class details view Init
	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bShowScrollBar = false;

	PopulationClassDetailsView = PropPlugin.CreateDetailView(DetailsViewArgs);
	PopulationClassDetailsView->SetObject(PopulationClass);

	// Tags tool Init
	TagsTool = SNew(SCustomizableObjectPopulationClassTagsTool).PopulationClass(PopulationClass);

	// Curve Editor Init
	CurveEditor = SNew(SCurveEditor)
		.ShowCurveSelector(true)
		.GridColor(FLinearColor(1.0f, 0.8f, 0.8f))
		.ViewMinInput(-0.01f)
		.ViewMaxInput(1.01f)
		.ViewMinOutput(-0.01f)
		.ViewMaxOutput(1.01f)
		.TimelineLength(1.01f)
		.HideUI(false)
		.ZoomToFitVertical(false)
		.ZoomToFitHorizontal(false)
		.AlwaysDisplayColorCurves(true);

	// Viewport Init
	Viewport = SNew(SCustomizableObjectPopulationEditorViewport);

	// Default Values
	TestPopulationInstancesNum = 10;
	PopulationClassAssetInstancesNum = 1;
	ViewportColumns = 5;
	InstanceSeparation = 100;

	// Tabs Manager distribution
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CustomizableObjectPopulationClassEditor_Layout_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.33f)
					->AddTab(PopulationClassPropertiesTabId, ETabState::OpenedTab)->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.33f)
					->AddTab(PopulationClassTagsToolId, ETabState::OpenedTab)->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.33f)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(CurveEditorTabId, ETabState::OpenedTab)->SetHideTabWell(true)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(ViewportTabId, ETabState::OpenedTab)->SetHideTabWell(true)
					)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, CustomizableObjectPopulationClassEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InObject);
}

FName FCustomizableObjectPopulationClassEditor::GetToolkitFName() const
{
	return FName("CustomizableObjectPopulationClassEditor");
}


FText FCustomizableObjectPopulationClassEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Customizable Object Population Class Editor");
}


FText FCustomizableObjectPopulationClassEditor::GetToolkitName() const
{
	return FText::FromString(GetEditingObject()->GetName());
}


FString FCustomizableObjectPopulationClassEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "CustomizableObjectPopulationClass").ToString();
}


FLinearColor FCustomizableObjectPopulationClassEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}


bool FCustomizableObjectPopulationClassEditor::IsTickable(void) const
{
	return true;
}


void FCustomizableObjectPopulationClassEditor::Tick(float InDeltaTime)
{
	// We need an Instance to get the decoration parameters
	if (PopulationClass->CustomizableObject)
	{
		if (PopulationClass->CustomizableObject->IsCompiled() && !PopulationClass->CustomizableObjectInstance)
		{
			PopulationClass->CustomizableObjectInstance = PopulationClass->CustomizableObject->CreateInstance();
			PopulationClass->CustomizableObjectInstance->SetBuildParameterRelevancy(true);
			PopulationClass->CustomizableObjectInstance->UpdateSkeletalMeshAsync(true, true);

			bRefreshDetailsView = true;
		}

		if (PopulationClass->CustomizableObjectInstance)
		{
			if (PopulationClass->CustomizableObjectInstance->GetCustomizableObject() != PopulationClass->CustomizableObject)
			{
				PopulationClass->CustomizableObjectInstance = nullptr;
			}
			else if (bRefreshDetailsView)
			{
				PopulationClassDetailsView->ForceRefresh();
				bRefreshDetailsView = false;
			}
		}
	}

	if (PopulationClass->EditorCurve)
	{
		if (CurrentEditorCurve != PopulationClass->EditorCurve)
		{
			CurrentEditorCurve = PopulationClass->EditorCurve;
			CurveEditor->SetCurveOwner(CurrentEditorCurve);
		}
	}
}


TStatId FCustomizableObjectPopulationClassEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCustomizableObjectPopulationClassEditor, STATGROUP_Tickables);
}


void FCustomizableObjectPopulationClassEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (PopulationClass)
	{
		Collector.AddReferencedObject(PopulationClass);
	
		if (PopulationClass->CustomizableObjectInstance)
		{
			Collector.AddReferencedObject(PopulationClass->CustomizableObjectInstance);
		}

		if (PopulationClass->EditorCurve)
		{
			Collector.AddReferencedObject(PopulationClass->EditorCurve);
		}
	}

	if (CurrentEditorCurve)
	{
		Collector.AddReferencedObject(CurrentEditorCurve);
	}

	for (int32 i = 0; i < PreviewCustomizableSkeletalComponents.Num(); ++i)
	{
		Collector.AddReferencedObject(PreviewCustomizableSkeletalComponents[i]);
	}
	
	for (int32 i = 0; i < PreviewSkeletalMeshComponents.Num(); ++i)
	{
		Collector.AddReferencedObject(PreviewSkeletalMeshComponents[i]);
	}
	
	for (int32 i = 0; i < ViewportInstances.Num(); ++i)
	{
		Collector.AddReferencedObject(ViewportInstances[i]);
	}
}


void FCustomizableObjectPopulationClassEditor::SetCustomAsset(UCustomizableObjectPopulationClass* InCustomAsset)
{
	PopulationClass = InCustomAsset;
}


TSharedRef<SDockTab> FCustomizableObjectPopulationClassEditor::SpawnTab_PopulationClassProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PopulationClassPropertiesTabId);

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
		.AlwaysShowScrollbar(false)
		.Thickness(FVector2D(12.0f, 12.0f))
		.HideWhenNotInUse(true);

	return SNew(SDockTab)
	.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PopulationClassEditorProperties_TabTitle", "Population Class Properties").ToString()))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SScrollBox)
			.ExternalScrollbar(ScrollBar)
			.ScrollBarAlwaysVisible(true)

			+ SScrollBox::Slot()
			[
				PopulationClassDetailsView.ToSharedRef()
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(FOptionalSize(16))
			[
				ScrollBar
			]
		]
	];
}


TSharedRef<SDockTab> FCustomizableObjectPopulationClassEditor::SpawnTab_PopulationClassTagsTool(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PopulationClassTagsToolId);

	return SNew(SDockTab)
	.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PopulationClassTagsTool_TagsTool", "Tags Tool").ToString()))
	[
		TagsTool.ToSharedRef()
	];
}


TSharedRef<SDockTab> FCustomizableObjectPopulationClassEditor::SpawnTab_CurveEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == CurveEditorTabId);

	return SNew(SDockTab)
	.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PoClassCurveEditor_Tab", "Curve Editor").ToString()))
	[
		CurveEditor.ToSharedRef()
	];
}

TSharedRef<SDockTab> FCustomizableObjectPopulationClassEditor::SpawnTab_Viewport(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == ViewportTabId);

	return SNew(SDockTab)
	.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PopulationClassViewport_Tab", "Viewport").ToString()))
	[
		Viewport.ToSharedRef()
	];
}


void FCustomizableObjectPopulationClassEditor::BindCommands()
{
	const FCustomizableObjectPopulationClassEditorCommands& Commands = FCustomizableObjectPopulationClassEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();
	
	UICommandList->MapAction(
	Commands.SaveCustomizableObject,
	FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SaveCustomizableObject),
	FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::CustomizableObjectCanBeSaved),
	FIsActionChecked());

	UICommandList->MapAction(
	Commands.OpenCustomizableObjectEditor,
	FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::OpenCustomizableObjectInEditor),
	FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::CanOpenCustomizableObjectEditor),
	FIsActionChecked());

	UICommandList->MapAction(
	Commands.SaveEditorCurve,
	FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::SaveEditorCurve),
	FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::EditorCurveCanBeSaved),
	FIsActionChecked());

	UICommandList->MapAction(
		Commands.TestPopulationClass,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::TestPopulationClass),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.GenerateInstances,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::GeneratePopulationClassInstances),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.InspectSelectedInstance,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::OpenInstance),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::CanOpenInstance),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.InspectSelectedSkeletalMesh,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::OpenSkeletalMesh),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationClassEditor::CanOpenInstance),
		FIsActionChecked());
	
}


void FCustomizableObjectPopulationClassEditor::SaveAsset_Execute()
{
	UPackage* Package = PopulationClass->GetOutermost();

	if (Package)
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(Package);

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);

		RecompileAllPopulations();
	}
}


bool FCustomizableObjectPopulationClassEditor::CustomizableObjectCanBeSaved() const
{
	if (PopulationClass && PopulationClass->CustomizableObject)
	{
		UPackage* Package = PopulationClass->CustomizableObject->GetOutermost();

		if (Package)
		{
			return Package->IsDirty();
		}
	}
	return false;
}


void FCustomizableObjectPopulationClassEditor::SaveCustomizableObject()
{
	UPackage* Package = PopulationClass->CustomizableObject->GetOutermost();

	if (Package)
	{
		TArray<UPackage*> PackagesToSave;
		PackagesToSave.Add(Package);

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);

		RecompileAllPopulations();
	}
}


void FCustomizableObjectPopulationClassEditor::RecompileAllPopulations()
{
	if (PopulationClass)
	{
		TArray<FName> ArrayReferenceNames;

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().GetReferencers(*PopulationClass->GetOuter()->GetPathName(), ArrayReferenceNames);

		FARFilter Filter;
		for (const FName& ReferenceName : ArrayReferenceNames)
		{
			if (!ReferenceName.ToString().StartsWith(TEXT("/TempAutosave")))
			{
				Filter.PackageNames.Add(ReferenceName);
			}
		}

		Filter.bIncludeOnlyOnDiskAssets = false;

		TArray<FAssetData> ArrayAssets;
		AssetRegistryModule.Get().GetAssets(Filter, ArrayAssets);

		for (int32 i = 0; i < ArrayAssets.Num(); ++i)
		{
			if (ArrayAssets[i].GetClass() == UCustomizableObjectPopulation::StaticClass())
			{
				if (ArrayAssets[i] != nullptr)
				{
					UCustomizableObjectPopulation* Population = Cast<UCustomizableObjectPopulation>((&ArrayAssets[i])->GetAsset());

					if (Population && Population->IsValidPopulation())
					{
						Population->CompilePopulation();
					}
				}
			}
		}
	}
}

bool FCustomizableObjectPopulationClassEditor::EditorCurveCanBeSaved() const
{
	if (PopulationClass->EditorCurve)
	{
		UPackage* Package = PopulationClass->EditorCurve->GetOutermost();

		if (Package)
		{
			return Package->IsDirty();
		}
	}

	return false;
}


void FCustomizableObjectPopulationClassEditor::SaveEditorCurve()
{
	if (PopulationClass->EditorCurve)
	{
		UPackage* Package = PopulationClass->EditorCurve->GetOutermost();

		if (Package)
		{
			TArray<UPackage*> PackagesToSave;
			PackagesToSave.Add(Package);

			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
		}
	}
}


void FCustomizableObjectPopulationClassEditor::ExpandToolBar()
{
	TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FCustomizableObjectPopulationClassEditor* Editor, TSharedPtr<FUICommandList> CommandList)
		{
			// Population actions

			// CUstomizable Object Management
			ToolbarBuilder.BeginSection("Customizable Object Management");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().SaveCustomizableObject);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().OpenCustomizableObjectEditor);

			ToolbarBuilder.EndSection();


			// Save Editor Curve
			ToolbarBuilder.BeginSection("Curve Editor");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().SaveEditorCurve);
			
			ToolbarBuilder.EndSection();


			// Test Population
			ToolbarBuilder.BeginSection("Test");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().TestPopulationClass);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(Editor, &FCustomizableObjectPopulationClassEditor::GenerateTestPopulationMenuContent, CommandList.ToSharedRef()),
				LOCTEXT("Test_Population_Class_Label", "Population Class Test Options"),
				LOCTEXT("Test_Population_Class_Tooltip", "Change Population Class Options"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.RepeatLastPlay"),
				true
			);

			// Generate Population Class Instances
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().GenerateInstances);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(Editor, &FCustomizableObjectPopulationClassEditor::GeneratePopulationClassInstancesMenuContent, CommandList.ToSharedRef()),
				LOCTEXT("Instrances_Population_Class_Label", "Population Class Instance Options"),
				LOCTEXT("Instrances_Population_Class_Tooltip", "Change Population Class Options"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.RepeatLastPlay"),
				true
			);

			ToolbarBuilder.EndSection();


			ToolbarBuilder.BeginSection("Instance Inspector");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().InspectSelectedInstance);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationClassEditorCommands::Get().InspectSelectedSkeletalMesh);

			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this, CommandList));

	AddToolbarExtender(ToolbarExtender);

	ICustomizableObjectPopulationEditorModule* CustomizableObjectPopulationEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectPopulationEditorModule>("CustomizableObjectPopulationEditor");
	AddToolbarExtender(CustomizableObjectPopulationEditorModule->GetCustomizableObjectPopulationEditorToolBarExtensibilityManager()->GetAllExtenders());
	
}


bool FCustomizableObjectPopulationClassEditor::CanOpenCustomizableObjectEditor()
{
	return PopulationClass->CustomizableObject != nullptr;
}


void FCustomizableObjectPopulationClassEditor::OpenCustomizableObjectInEditor()
{
	if (PopulationClass && PopulationClass->CustomizableObject)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(PopulationClass->CustomizableObject);
	}
}


TSharedRef<SWidget> FCustomizableObjectPopulationClassEditor::GenerateTestPopulationMenuContent(TSharedRef<FUICommandList> InCommandList)
{

	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	// settings
	MenuBuilder.BeginSection("Options", LOCTEXT("PopulationClassTestOptions", "Options"));
	{
		TestPopulationInstancesNumEntry = SNew(SNumericEntryBox<int32>)
		.AllowSpin(true)
		.MinValue(1)
		.MaxValue(100)
		.MinSliderValue(1)
		.MaxSliderValue(100)
		.Value(this, &FCustomizableObjectPopulationClassEditor::GetTestPopulationInstancesNum)
		.OnValueChanged(this, &FCustomizableObjectPopulationClassEditor::OnTestPopulationInstancesNumChanged);

		MenuBuilder.AddWidget(TestPopulationInstancesNumEntry.ToSharedRef(), LOCTEXT("PopulationClassTestCount", "Instance Count"));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Viewport Settings", LOCTEXT("PopulationClassTestViewportSettingsOptions", "Viewport Settings"));
	{
		ViewportColumnsEntry = SNew(SNumericEntryBox<int32>).AllowSpin(true)
		.MinValue(1).MaxValue(100)
		.MinSliderValue(1).MaxSliderValue(100)
		.Value_Lambda([this]()->int32 {return ViewportColumns; })
		.OnValueChanged_Lambda([this](int32 NewValue) {ViewportColumns = NewValue; });

		MenuBuilder.AddWidget(ViewportColumnsEntry.ToSharedRef(), LOCTEXT("PopulationClassColumns", "Population Columns:"));

		InstanceSeparationEntry = SNew(SNumericEntryBox<int32>).AllowSpin(true)
		.MinValue(1).MaxValue(1000)
		.MinSliderValue(1).MaxSliderValue(1000)
		.Value_Lambda([this]()->int32 {return InstanceSeparation; })
		.OnValueChanged_Lambda([this](int32 NewValue) {InstanceSeparation = NewValue; });

		MenuBuilder.AddWidget(InstanceSeparationEntry.ToSharedRef(), LOCTEXT("PopulationClassSeparation", "Instance separation:"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


TOptional<int32> FCustomizableObjectPopulationClassEditor::GetTestPopulationInstancesNum() const
{
	return TestPopulationInstancesNum;
}


void FCustomizableObjectPopulationClassEditor::OnTestPopulationInstancesNumChanged(int32 Value)
{
	TestPopulationInstancesNum = Value;
}


void FCustomizableObjectPopulationClassEditor::TestPopulationClass()
{
	if (PopulationClass && Viewport.IsValid())
	{
		PreviewCustomizableSkeletalComponents.Empty();
		PreviewSkeletalMeshComponents.Empty();
		ViewportInstances.Empty();
		ColliderComponents.Empty();
	
		// Creating a Population
		UCustomizableObjectPopulation* Population = nullptr;
		Population = NewObject<UCustomizableObjectPopulation>();
		
		if (Population)
		{
			FClassWeightPair ClassWeightPair;
			ClassWeightPair.Class = PopulationClass;
			ClassWeightPair.ClassWeight = 1;
	
			Population->ClassWeights.Emplace(ClassWeightPair);
			Population->CompilePopulation(NewObject<UCustomizableObjectPopulationGenerator>());
				
			if (Population->IsValidPopulation())
			{
				// Creating population instances
				Population->GeneratePopulation(MutableView(ViewportInstances), TestPopulationInstancesNum);
	
				if (ViewportInstances.Num() > 0)
				{
					for (int32 i = 0; i < ViewportInstances.Num(); ++i)
					{
						const UCustomizableObject* CustomizableObject = ViewportInstances[i]->GetCustomizableObject();	
						if (!CustomizableObject)
						{
							continue;
						}

						// Creating the Customizable Skeletal Component
						UCustomizableSkeletalComponent* PreviewCustomizableSkeletalComponent = nullptr;
						PreviewCustomizableSkeletalComponent = NewObject<UCustomizableSkeletalComponent>(UCustomizableSkeletalComponent::StaticClass());
	
						// Creating the Skeletal Mesh Component
						USkeletalMeshComponent* SkeletalMeshComponent = nullptr;
						SkeletalMeshComponent = NewObject<USkeletalMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
						
						// Creating a Collider Component for mouse picking
						UCapsuleComponent* ColliderComponent = nullptr;
						ColliderComponent = NewObject<UCapsuleComponent>(GetTransientPackage(), NAME_None, RF_Transient);
	
						// Attaching the Customizable Skeletal Component to the Skeletal Mesh Component
						if (PreviewCustomizableSkeletalComponent && SkeletalMeshComponent && ColliderComponent)
						{
							ViewportInstances[i]->SetBuildParameterRelevancy(true);
							ViewportInstances[i]->UpdateSkeletalMeshAsync(true, true);

							PreviewCustomizableSkeletalComponent->SetCustomizableObjectInstance(ViewportInstances[i]);
							// TODO: UE-274335. We want this code to be able to operate with multiple components, not just the first one
							PreviewCustomizableSkeletalComponent->SetComponentName(CustomizableObject->GetComponentName(0));
							PreviewCustomizableSkeletalComponent->AttachToComponent(SkeletalMeshComponent, FAttachmentTransformRules::KeepRelativeTransform);

							ColliderComponent->AttachToComponent(SkeletalMeshComponent, FAttachmentTransformRules::KeepRelativeTransform);

							PreviewCustomizableSkeletalComponents.Add(PreviewCustomizableSkeletalComponent);
							PreviewSkeletalMeshComponents.Add(SkeletalMeshComponent);
							ColliderComponents.Add(ColliderComponent);
						}
					}
	
					Viewport->SetPreviewComponents(PreviewSkeletalMeshComponents, ColliderComponents, ViewportColumns, InstanceSeparation);
				}
			}
		}
	}
}


TSharedRef<SWidget> FCustomizableObjectPopulationClassEditor::GeneratePopulationClassInstancesMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	// settings
	MenuBuilder.BeginSection("Options", LOCTEXT("CreateInstances", "Options"));
	{
		PopulationClassAssetInstancesNumEntry = SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.MinValue(1)
			.MaxValue(100)
			.MinSliderValue(1)
			.MaxSliderValue(100)
			.Value(this, &FCustomizableObjectPopulationClassEditor::GetPopulationClassAssetInstancesNum)
			.OnValueChanged(this, &FCustomizableObjectPopulationClassEditor::OnPopulationClassAssetInstancesNumChanged);

		MenuBuilder.AddWidget(PopulationClassAssetInstancesNumEntry.ToSharedRef(), LOCTEXT("PopulationAssetInstancesCount", "Instance Count"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


TOptional<int32> FCustomizableObjectPopulationClassEditor::GetPopulationClassAssetInstancesNum() const
{
	return PopulationClassAssetInstancesNum;
}


void FCustomizableObjectPopulationClassEditor::OnPopulationClassAssetInstancesNumChanged(int32 Value)
{
	PopulationClassAssetInstancesNum = Value;
}


bool FCustomizableObjectPopulationClassEditor::CanOpenInstance()
{
	if (Viewport.IsValid())
	{
		int32 InstanceIndex = Viewport->GetSelectedInstance();
		return  ViewportInstances.IsValidIndex(InstanceIndex);
	}

	return false;
}


void FCustomizableObjectPopulationClassEditor::OpenInstance()
{
	int32 SelectedInstance = Viewport->GetSelectedInstance();

	if (ViewportInstances[SelectedInstance])
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ViewportInstances[SelectedInstance]);
	}
}


void FCustomizableObjectPopulationClassEditor::OpenSkeletalMesh()
{
	int32 SelectedInstance = Viewport->GetSelectedInstance();

	if (!ViewportInstances[SelectedInstance])
	{
		return;
	}
	
	TMap<FName, TObjectPtr<USkeletalMesh>>::TConstIterator It = ViewportInstances[SelectedInstance]->GetPrivate()->SkeletalMeshes.CreateConstIterator();
	if (!It)
	{
		return;
	}
	
	USkeletalMesh* SkeletalMesh = It->Value;
	if (!SkeletalMesh)
	{
		return;
	}	

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SkeletalMesh);
}


class SSelectPopulationClassFolderDlg : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSelectPopulationClassFolderDlg) {}
		SLATE_ARGUMENT(FText, DefaultAssetPath)
		SLATE_ARGUMENT(FText, DefaultFileName)
	SLATE_END_ARGS()

	SSelectPopulationClassFolderDlg() : UserResponse(EAppReturnType::Cancel) {}

	void Construct(const FArguments& InArgs);

public:

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

	/** FileName getter */
	FString GetFileName();

protected:

	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	void OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo);

	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText FileName;

};


void FCustomizableObjectPopulationClassEditor::GeneratePopulationClassInstances()
{
	UCustomizableObjectPopulation* Population = nullptr;
	Population = NewObject<UCustomizableObjectPopulation>();

	if (!Population)
	{
		return;
	}
	
	// Generating & compiling class
	FClassWeightPair ClassWeightPair;
	ClassWeightPair.Class = PopulationClass;
	ClassWeightPair.ClassWeight = 1;

	Population->ClassWeights.Emplace(ClassWeightPair);
	Population->CompilePopulation(NewObject<UCustomizableObjectPopulationGenerator>());

	// Creating popup window
	FString DefaultName = PopulationClass->Name;
	FText DefaultFileName = FText::Format(LOCTEXT("DefaultPopulationClassFileName", "{0}_Inst"),
		FText::FromString(DefaultName));

	TSharedRef<SSelectPopulationClassFolderDlg> FolderDlg =
		SNew(SSelectPopulationClassFolderDlg)
		.DefaultAssetPath(FText())
		.DefaultFileName(DefaultFileName);

	// Popup window management
	if (FolderDlg->ShowModal() != EAppReturnType::Cancel)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		// overwrite window default value
		EAppReturnType::Type RetType = EAppReturnType::No;

		TArray<UCustomizableObjectInstance*> Instances;
		TArray<UCustomizableObjectInstance*> ExistingInstances;
		TArray<UObject*> ObjectsToSync;
		TArray<UPackage*> PackagesToSave;

		for (int32 i = 0; i < PopulationClassAssetInstancesNum; ++i)
		{
			FString ObjectName = FolderDlg->GetFileName();

			if (i > 0)
			{
				ObjectName += FString::Printf(TEXT("_%d"), i);
			}

			FString FilePath = FolderDlg->GetAssetPath() + FString("/") + ObjectName;
			FAssetData AssetData;

			if (RetType == EAppReturnType::No || RetType == EAppReturnType::Yes || RetType == EAppReturnType::YesAll)
			{
				//Search for asset with the same name
				FString ObjectPath = FilePath + FString(".") + ObjectName;
				AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));

				// Popup window
				if (AssetData.IsValid() && AssetData.GetClass() == UCustomizableObjectInstance::StaticClass() && RetType != EAppReturnType::YesAll)
				{
					FText Msg = FText::Format(LOCTEXT("FCustomizablePopulationClassEditorViewportClient_CreateInstance_Overwrite", "There is already an instance with name: '{0}'. Do you want to overwrite it?"),
						FText::FromString(ObjectName));
					RetType = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel, Msg);
				}
			}

			if (RetType == EAppReturnType::Cancel)
			{
				return;
			}

			UCustomizableObjectInstance* Instance = nullptr;

			if (RetType == EAppReturnType::No || RetType == EAppReturnType::NoAll || !AssetData.IsValid())
			{
				if (AssetData.IsValid() && i == 0)
				{
					FilePath += "_0";
				}

				FString PackageName;
				FString Name;

				// Determine an appropriate name
				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateUniqueAssetName(FilePath, "", PackageName, Name);

				UPackage* Pkg = CreatePackage(*PackageName);
				PackagesToSave.Add(Pkg);
				Instance = NewObject<UCustomizableObjectInstance>(Pkg, FName(*Name), RF_Public | RF_Standalone);
				Pkg->MarkPackageDirty();
				ObjectsToSync.Add(Instance);
			}
			else
			{
				UCustomizableObjectInstance* ExistingInstance = Cast<UCustomizableObjectInstance>(AssetData.GetAsset());
				ExistingInstances.Add(ExistingInstance);

				Instance = NewObject<UCustomizableObjectInstance>(GetTransientPackage());	
			}

			if (Instance)
			{
				Instances.Add(Instance);
			}
		}

		if (Instances.Num() > 0)
		{
			Population->GeneratePopulation(Instances, PopulationClassAssetInstancesNum);
			int32 ExistingInstancesIndx = 0;

			for (int32 i = 0; i < Instances.Num(); ++i)
			{
				// Update instances to overwrite
				if (Instances[i]->GetOutermost()->HasAnyFlags(EObjectFlags::RF_Transient))
				{
					ExistingInstances[ExistingInstancesIndx] = Instances[i]->Clone();
					PackagesToSave.Add(ExistingInstances[ExistingInstancesIndx]->GetPackage());
					ExistingInstancesIndx++;
				}
				else
				{
					//Notify that an asset has been created
					FAssetRegistryModule::AssetCreated(Instances[i]);
				}
			}

			if (ObjectsToSync.Num() > 0)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
			}

			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, true);
		}
	}
}


/////////////////////////////////////////////////
// select folder dialog
//////////////////////////////////////////////////
void SSelectPopulationClassFolderDlg::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));
	FileName = InArgs._DefaultFileName;

	if (AssetPath.IsEmpty())
	{
		AssetPath = FText::FromString(TEXT("/Game"));
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SSelectPopulationClassFolderDlg::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("SSelectFolderDlg_Title", "Select target folder to store the instances"))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	//.SizingRule( ESizingRule::Autosized )
	.ClientSize(FVector2D(450, 450))
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Add user input block
		.Padding(2)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectPath", "Select Path"))
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 14, "Regular"))
				]

				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(3)
				[
					ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FileName", "File Name"))
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 14, "Regular"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SEditableTextBox)
					.Text(InArgs._DefaultFileName)
					.OnTextCommitted(this, &SSelectPopulationClassFolderDlg::OnNameChange)
					.MinDesiredWidth(250)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))

			+ SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("OK", "OK"))
				.OnClicked(this, &SSelectPopulationClassFolderDlg::OnButtonClick, EAppReturnType::Ok)
			]

			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SSelectPopulationClassFolderDlg::OnButtonClick, EAppReturnType::Cancel)
			]
		]
	]);
}

void SSelectPopulationClassFolderDlg::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
}

FReply SSelectPopulationClassFolderDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	RequestDestroyWindow();

	return FReply::Handled();
}


void SSelectPopulationClassFolderDlg::OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo)
{
	FileName = NewName;
}


EAppReturnType::Type SSelectPopulationClassFolderDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SSelectPopulationClassFolderDlg::GetAssetPath()
{
	return AssetPath.ToString();
}


FString SSelectPopulationClassFolderDlg::GetFileName()
{
	return FileName.ToString();
}


#undef LOCTEXT_NAMESPACE
