// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "MuCOPE/ICustomizableObjectPopulationClassEditor.h"
#include "TickableEditorObject.h"

class FReferenceCollector;
class FSpawnTabArgs;
class FUICommandList;
class IToolkitHost;
class SDockTab;
class SWidget;
class UCurveBase;
class UCustomizableObjectInstance;
class UCustomizableObjectPopulationClass;
template <typename NumericType> class SNumericEntryBox;

class FCustomizableObjectPopulationClassEditor : 
	public ICustomizableObjectPopulationClassEditor,
	public FEditorUndoClient,
	public FNotifyHook,
	public FTickableEditorObject,
	public FGCObject
{
public:

	FCustomizableObjectPopulationClassEditor();
	~FCustomizableObjectPopulationClassEditor();

	/** FGCObjet functions */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	/** Editor init */
	void InitCustomizableObjectPopulationClassEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulationClass* InObject);

	// IToolkit interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** FTickableGameObject interface */
	virtual bool IsTickable(void) const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Begin ICustomAssetEditor initerface */
	virtual UCustomizableObjectPopulationClass* GetCustomAsset() { return PopulationClass; }
	virtual void SetCustomAsset(UCustomizableObjectPopulationClass* InCustomAsset) override;

	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomizableObjectPopulationClassEditor");
	}

private:

	/** Tab Spawners */
	TSharedRef<SDockTab> SpawnTab_PopulationClassProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PopulationClassTagsTool(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_CurveEditor(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);

private:

	/** Binds commands associated with the Static Mesh Editor. */
	void BindCommands();

	/** Adds the Customizable Object Population Editor commands to the default toolbar */
	void ExpandToolBar();

	/** Save Customizable Object Population Class open in editor */
	void SaveAsset_Execute() override;

	bool CustomizableObjectCanBeSaved() const;
	void SaveCustomizableObject();
	bool EditorCurveCanBeSaved() const;
	void SaveEditorCurve();
	/** **/

	/** Instance inspection functions*/
	bool CanOpenInstance();
	void OpenInstance();
	void OpenSkeletalMesh();
	/** **/

	/** Opens the Customizable Object into an editor */
	bool CanOpenCustomizableObjectEditor();
	void OpenCustomizableObjectInEditor();

	/** Generate instances in the viewport to test the population */
	void TestPopulationClass();

	TSharedRef<SWidget> GenerateTestPopulationMenuContent(TSharedRef<FUICommandList> InCommandList);

	TOptional<int32> GetTestPopulationInstancesNum() const;
	void OnTestPopulationInstancesNumChanged(int32 Value);

	TSharedRef<SWidget> GeneratePopulationClassInstancesMenuContent(TSharedRef<FUICommandList> InCommandList);
	TOptional<int32> GetPopulationClassAssetInstancesNum() const;
	void OnPopulationClassAssetInstancesNumChanged(int32 Value);

	/** Generates instance assets of the population */
	void GeneratePopulationClassInstances();

	/** Recompile all the populations where this class is referenced */
	void RecompileAllPopulations();

private:

	/** Pointer to the Population Class open in this editor */
	TObjectPtr<UCustomizableObjectPopulationClass> PopulationClass;

	/** Tabs IDs */
	static const FName PopulationClassPropertiesTabId;
	static const FName PopulationClassTagsToolId;
	static const FName CurveEditorTabId;
	static const FName ViewportTabId;

	/** Population Class Property View */
	TSharedPtr<class IDetailsView> PopulationClassDetailsView;

	/** Tags Tool */
	TSharedPtr<class SCustomizableObjectPopulationClassTagsTool> TagsTool;

	/** Curve editor to edit the curve of a population class characteristic */
	TSharedPtr<class SCurveEditor> CurveEditor;

	class FCustomizableObjectPopulationClassDetails* PopulationClassDetailsPtr;

	/** Current curve shown in editor */
	TObjectPtr<UCurveBase> CurrentEditorCurve;

	/** Indicates when the Details View needs to be refreshed */
	bool bRefreshDetailsView;

	///** Editor Viewport */
	TSharedPtr<class SCustomizableObjectPopulationEditorViewport> Viewport;
	
	// Number of instances that will be generated in the preview viewport
	int32 TestPopulationInstancesNum;

	// Number of instances that will be generated and saved
	int32 PopulationClassAssetInstancesNum;

	// Box to select the number of instances spawned in the viewport
	TSharedPtr< SNumericEntryBox<int32> > TestPopulationInstancesNumEntry;

	// Box to select the number of instances to save
	TSharedPtr< SNumericEntryBox<int32> > PopulationClassAssetInstancesNumEntry;

	//Viewport instance configuration
	int32 ViewportColumns;
	int32 InstanceSeparation;
	TSharedPtr< SNumericEntryBox<int32> > ViewportColumnsEntry;
	TSharedPtr< SNumericEntryBox<int32> > InstanceSeparationEntry;
	
	// Components to preview instances on the viewport
	TArray<TObjectPtr<class UCustomizableSkeletalComponent>> PreviewCustomizableSkeletalComponents;
	TArray<TObjectPtr<class USkeletalMeshComponent>> PreviewSkeletalMeshComponents;
	TArray<class UCapsuleComponent*> ColliderComponents;
	TArray<TObjectPtr<UCustomizableObjectInstance>> ViewportInstances;
	//

};
