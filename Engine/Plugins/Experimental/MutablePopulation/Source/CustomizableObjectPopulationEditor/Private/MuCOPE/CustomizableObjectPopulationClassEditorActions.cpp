// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationClassEditorActions.h"

#include "MuCOPE/CustomizableObjectPopulationEditorStyle.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectEditor"

FCustomizableObjectPopulationClassEditorCommands::FCustomizableObjectPopulationClassEditorCommands() : TCommands<FCustomizableObjectPopulationClassEditorCommands>
(
	"CustomizableObjectPopulationClassEditor", // Context name for fast lookup
	NSLOCTEXT("Contexts", "CustomizableObjectPopulationClassEditor", "CustomizableObject Population Class Editor"), // Localized context name for displaying
	NAME_None, // Parent
	FCustomizableObjectPopulationEditorStyle::GetStyleSetName()
	)
{

}

void FCustomizableObjectPopulationClassEditorCommands::RegisterCommands()
{
	UI_COMMAND(Save, "Save", "Save the Population Class.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveCustomizableObject, "Save CO", "Save the Customizable Object of the Population Class", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(OpenCustomizableObjectEditor, "Open Editor", "Open the Customizable Object Editor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SaveEditorCurve, "Save Editor Curve", "Save the Curve Asset Open in the Editor", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TestPopulationClass, "Test Population Class", "Generate intances in the viewport to test the Population Class", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GenerateInstances, "Generate Instances", "Generate random instances generated by this Population Class", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InspectSelectedInstance, "Inspect Instance", "Open the Customizable Object Instance Editor to inspect the selected instance in the preview viewport", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InspectSelectedSkeletalMesh, "Inspect Skeletal Mesh", "Open the skeletal mesh editor to inspect the selected instance in the preview viewport", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
