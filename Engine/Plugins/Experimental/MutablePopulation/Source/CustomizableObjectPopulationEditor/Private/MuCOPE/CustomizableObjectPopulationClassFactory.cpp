// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationClassFactory.h"

#include "MuCOP/CustomizableObjectPopulationClass.h"

class FFeedbackContext;
class UClass;
class UObject;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClassFactory"

UCustomizableObjectPopulationClassFactory::UCustomizableObjectPopulationClassFactory() : Super()
{
	// Property initialization
	bCreateNew = true;
	SupportedClass = UCustomizableObjectPopulationClass::StaticClass();
	bEditAfterNew = true;
}

 UObject* UCustomizableObjectPopulationClassFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	 UCustomizableObjectPopulationClass* CustomizableObjectPopulation = NewObject<UCustomizableObjectPopulationClass>(InParent, Class, Name, Flags);
	 
	 return CustomizableObjectPopulation;
}

 bool UCustomizableObjectPopulationClassFactory::ShouldShowInNewMenu() const
 {
	 return true;
 }

#undef LOCTEXT_NAMESPACE
