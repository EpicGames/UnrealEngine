// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationEditor.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ContentBrowserModule.h"
#include "DetailsViewArgs.h"
#include "Engine/SkeletalMesh.h"
#include "FileHelpers.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "IDetailsView.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCOP/CustomizableObjectPopulation.h"
#include "MuCOP/CustomizableObjectPopulationGenerator.h"
#include "MuCOPE/CustomizableObjectPopulationEditorActions.h"
#include "MuCOPE/CustomizableObjectPopulationEditorModule.h"
#include "MuCOPE/SCustomizableObjectPopulationEditorViewport.h"
#include "PropertyEditorModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "MuCO/CustomizableObjectInstancePrivate.h"

class IToolkitHost;
class SWidget;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationEditor"

const FName FCustomizableObjectPopulationEditor::PopulationPropertiesTabId(TEXT("PopulationEditor_Properties"));
const FName FCustomizableObjectPopulationEditor::PopulationViewportTabId(TEXT("PopulationEditor_Viewport"));


void FCustomizableObjectPopulationEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	InTabManager->RegisterTabSpawner(PopulationPropertiesTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationEditor::SpawnTab_PopulationProperties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Properties"))
		.SetGroup(MenuStructure.GetToolsCategory());

	InTabManager->RegisterTabSpawner(PopulationViewportTabId, FOnSpawnTab::CreateSP(this, &FCustomizableObjectPopulationEditor::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(MenuStructure.GetToolsCategory());
}


void FCustomizableObjectPopulationEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(PopulationPropertiesTabId);
	InTabManager->UnregisterTabSpawner(PopulationViewportTabId);
}


FCustomizableObjectPopulationEditor::FCustomizableObjectPopulationEditor()
{
	Population = nullptr;
}


FCustomizableObjectPopulationEditor::~FCustomizableObjectPopulationEditor()
{
	Population = nullptr;
}


void FCustomizableObjectPopulationEditor::InitCustomizableObjectPopulationEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, UCustomizableObjectPopulation* InObject)
{
	Population = InObject;

	FCustomizableObjectPopulationEditorCommands::Register();
	BindCommands();

	// Population class details view Init
	FPropertyEditorModule& PropPlugin = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NotifyHook = this;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowObjectLabel = false;

	PopulationDetailsView = PropPlugin.CreateDetailView(DetailsViewArgs);
	PopulationDetailsView->SetObject(Population);

	Viewport = SNew(SCustomizableObjectPopulationEditorViewport);

	//Default values
	TestPopulationInstancesNum = 10;
	PopulationAssetInstancesNum = 1;
	ViewportColumns = 5;
	InstanceSeparation = 100;

	// Tabs Manager distribution
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CustomizableObjectPopulationEditor_Layout_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(PopulationPropertiesTabId, ETabState::OpenedTab)->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.5f)
					->AddTab(PopulationViewportTabId, ETabState::OpenedTab)->SetHideTabWell(true)
				)
			)
		);

	ExtendToolbar();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, CustomizableObjectPopulationEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InObject);
}

FName FCustomizableObjectPopulationEditor::GetToolkitFName() const
{
	return FName("CustomizableObjectPopulationEditor");
}


FText FCustomizableObjectPopulationEditor::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Customizable Object Population Editor");
}


FText FCustomizableObjectPopulationEditor::GetToolkitName() const
{
	return FText::FromString(GetEditingObject()->GetName());
}


FString FCustomizableObjectPopulationEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "CustomizableObjectPopulation").ToString();
}


FLinearColor FCustomizableObjectPopulationEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}


bool FCustomizableObjectPopulationEditor::IsTickable(void) const
{
	return true;
}


void FCustomizableObjectPopulationEditor::Tick(float InDeltaTime)
{
	
}


TStatId FCustomizableObjectPopulationEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FCustomizableObjectPopulationEditor, STATGROUP_Tickables);
}


void FCustomizableObjectPopulationEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Population);
}


void FCustomizableObjectPopulationEditor::SetCustomAsset(UCustomizableObjectPopulation* InCustomAsset)
{
	Population = InCustomAsset;
}


TSharedRef<SDockTab> FCustomizableObjectPopulationEditor::SpawnTab_PopulationProperties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PopulationPropertiesTabId);

	return SNew(SDockTab)
		.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PopulationEditorProperties_TabTitle", "Population Properties").ToString()))
		[
			PopulationDetailsView.ToSharedRef()
		];
}


TSharedRef<SDockTab> FCustomizableObjectPopulationEditor::SpawnTab_Viewport(const FSpawnTabArgs & Args)
{
	check(Args.GetTabId() == PopulationViewportTabId);

	return SNew(SDockTab)
	.Label(FText::FromString(GetTabPrefix() + LOCTEXT("PopulationEditorViewport_TabTitle", "Population Viewport").ToString()))
	[
		Viewport.ToSharedRef()
	];
}


void FCustomizableObjectPopulationEditor::BindCommands()
{
	const FCustomizableObjectPopulationEditorCommands& Commands = FCustomizableObjectPopulationEditorCommands::Get();

	const TSharedRef<FUICommandList>& UICommandList = GetToolkitCommands();

	// Toolbar
	UICommandList->MapAction(
		Commands.TestPopulation,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::TestPopulation),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.GenerateInstances,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::GeneratePopulationInstances),
		FIsActionChecked());

	UICommandList->MapAction(
		Commands.InspectSelectedInstance,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::OpenInstance),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::CanOpenInstance),
		FIsActionChecked());
	
	UICommandList->MapAction(
		Commands.InspectSelectedSkeletalMesh,
		FExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::OpenSkeletalMesh),
		FCanExecuteAction::CreateSP(this, &FCustomizableObjectPopulationEditor::CanOpenInstance),
		FIsActionChecked());
}


void FCustomizableObjectPopulationEditor::SaveAsset_Execute()
{
	if (Population && Population->IsValidPopulation())
	{
		if (UPackage* Package = Population->GetOutermost())
		{
			Population->CompilePopulation(NewObject<UCustomizableObjectPopulationGenerator>(Package, FName("PopulationGenerator")));

			TArray<UPackage*> PackagesToSave;
			PackagesToSave.Add(Package);

			Population->MarkPackageDirty();

			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
		}
	}
}


void FCustomizableObjectPopulationEditor::ExtendToolbar()
{
	TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();

	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder, FCustomizableObjectPopulationEditor* Editor, TSharedPtr<FUICommandList> CommandList)
		{
			// Population actions
			// Test Population
			ToolbarBuilder.BeginSection("Test");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationEditorCommands::Get().TestPopulation);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(Editor, &FCustomizableObjectPopulationEditor::GenerateTestPopulationMenuContent, CommandList.ToSharedRef()),
				LOCTEXT("Test_Population_Label", "Population Test Options"),
				LOCTEXT("Test_Population_Tooltip", "Change Population Options"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.RepeatLastPlay"),
				true
			);

			// Generate Instances
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationEditorCommands::Get().GenerateInstances);
			ToolbarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateSP(Editor, &FCustomizableObjectPopulationEditor::GeneratePopulationInstancesMenuContent, CommandList.ToSharedRef()),
				LOCTEXT("Population_Assets_Label", "Population Assets Options"),
				LOCTEXT("Population_Assets_Tooltip", "Change Population Options"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.RepeatLastPlay"),
				true
			);

			ToolbarBuilder.EndSection();


			ToolbarBuilder.BeginSection("Instance Inspector");

			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationEditorCommands::Get().InspectSelectedInstance);
			ToolbarBuilder.AddToolBarButton(FCustomizableObjectPopulationEditorCommands::Get().InspectSelectedSkeletalMesh);

			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic(&Local::FillToolbar, this, CommandList));

	AddToolbarExtender(ToolbarExtender);

	ICustomizableObjectPopulationEditorModule* CustomizableObjectPopulationEditorModule = &FModuleManager::LoadModuleChecked<ICustomizableObjectPopulationEditorModule>("CustomizableObjectPopulationEditor");
	AddToolbarExtender(CustomizableObjectPopulationEditorModule->GetCustomizableObjectPopulationEditorToolBarExtensibilityManager()->GetAllExtenders());
}


TSharedRef<SWidget> FCustomizableObjectPopulationEditor::GenerateTestPopulationMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	// settings
	MenuBuilder.BeginSection("Options", LOCTEXT("TestPopulation", "Options"));
	{
		TestPopulationInstancesNumEntry = SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.MinValue(1)
			.MaxValue(100)
			.MinSliderValue(1)
			.MaxSliderValue(100)
			.Value(this, &FCustomizableObjectPopulationEditor::GetTestPopulationInstancesNum)
			.OnValueChanged(this, &FCustomizableObjectPopulationEditor::OnTestPopulationInstancesNumChanged);

		MenuBuilder.AddWidget(TestPopulationInstancesNumEntry.ToSharedRef(), LOCTEXT("MutableStressTestInstanceCount", "Instance Count"));
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Viewport Settings", LOCTEXT("PopulationClassTestViewportSettingsOptions", "Viewport Settings"));
	{
		ViewportColumnsEntry = SNew(SNumericEntryBox<int32>).AllowSpin(true)
		.MinValue(1).MaxValue(100)
		.MinSliderValue(1).MaxSliderValue(100)
		.Value_Lambda([this]()->int32 {return ViewportColumns; })
		.OnValueChanged_Lambda([this](int32 NewValue) {ViewportColumns = NewValue; });

		MenuBuilder.AddWidget(ViewportColumnsEntry.ToSharedRef(), LOCTEXT("PopulationClassColumns", "Population Columns:"));

		InstanceSeparationEntry = SNew(SNumericEntryBox<int32>).AllowSpin(true)
		.MinValue(1).MaxValue(1000)
		.MinSliderValue(1).MaxSliderValue(1000)
		.Value_Lambda([this]()->int32 {return InstanceSeparation; })
		.OnValueChanged_Lambda([this](int32 NewValue) {InstanceSeparation = NewValue; });

		MenuBuilder.AddWidget(InstanceSeparationEntry.ToSharedRef(), LOCTEXT("PopulationClassSeparation", "Instance separation:"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


TOptional<int32> FCustomizableObjectPopulationEditor::GetTestPopulationInstancesNum() const
{
	return TestPopulationInstancesNum;
}


void FCustomizableObjectPopulationEditor::OnTestPopulationInstancesNumChanged(int32 Value)
{
	TestPopulationInstancesNum = Value;
}


TSharedRef<SWidget> FCustomizableObjectPopulationEditor::GeneratePopulationInstancesMenuContent(TSharedRef<FUICommandList> InCommandList)
{
	const bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

	// settings
	MenuBuilder.BeginSection("Options", LOCTEXT("CreateInstances", "Options"));
	{
		PopulationAssetInstancesNumEntry = SNew(SNumericEntryBox<int32>)
			.AllowSpin(true)
			.MinValue(1)
			.MaxValue(100)
			.MinSliderValue(1)
			.MaxSliderValue(100)
			.Value(this, &FCustomizableObjectPopulationEditor::GetPopulationAssetInstancesNum)
			.OnValueChanged(this, &FCustomizableObjectPopulationEditor::OnPopulationAssetInstancesNumChanged);

		MenuBuilder.AddWidget(PopulationAssetInstancesNumEntry.ToSharedRef(), LOCTEXT("PopulationAssetInstancesCount", "Instance Count"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


TOptional<int32> FCustomizableObjectPopulationEditor::GetPopulationAssetInstancesNum() const
{
	return PopulationAssetInstancesNum;
}


void FCustomizableObjectPopulationEditor::OnPopulationAssetInstancesNumChanged(int32 Value)
{
	PopulationAssetInstancesNum = Value;
}


void FCustomizableObjectPopulationEditor::TestPopulation()
{
	PreviewCustomizableSkeletalComponents.Empty();
	PreviewSkeletalMeshComponents.Empty();
	ViewportInstances.Empty();
	ColliderComponents.Empty();

	if (Population && Viewport.IsValid())
	{
		if (!Population->IsValidPopulation())
		{
			UE_LOGF(LogMutable, Warning, "There are one or more unassigned Population Classes. Please review your %ls Population.", *(Population->Name));
			return;
		}

		if (!Population->HasGenerator())
		{
			UE_LOGF(LogMutable, Warning, "Save the population before testing it.");
			return;
		}
		
		Population->CompilePopulation();

		// Creating the population instances
		Population->GeneratePopulation(ViewportInstances, TestPopulationInstancesNum);

		if (ViewportInstances.Num() > 0)
		{
			for (int32 i = 0; i < ViewportInstances.Num(); ++i)
			{
				const UCustomizableObject* CustomizableObject = ViewportInstances[i]->GetCustomizableObject();	
				if (!CustomizableObject)
				{
					continue;
				}

				// Creating the Customizable Skeletal Component
				UCustomizableSkeletalComponent* PreviewCustomizableSkeletalComponent = nullptr;
				PreviewCustomizableSkeletalComponent = NewObject<UCustomizableSkeletalComponent>(UCustomizableSkeletalComponent::StaticClass());
				
				// Creating the Skeletal Mesh Component
				USkeletalMeshComponent* PreviewSkeletalMeshComponent = nullptr;
				PreviewSkeletalMeshComponent = NewObject<USkeletalMeshComponent>(GetTransientPackage(), NAME_None, RF_Transient);
				
				// Creating the Collider Component for mouse picking
				UCapsuleComponent* ColliderComponent = nullptr;
				ColliderComponent = NewObject<UCapsuleComponent>(GetTransientPackage(), NAME_None, RF_Transient);

				// Attaching the Customizable Skeletal Component to the Skeletal Mesh Component
				if (PreviewCustomizableSkeletalComponent && PreviewSkeletalMeshComponent && ColliderComponent)
				{
					ViewportInstances[i]->SetBuildParameterRelevancy(true);
					ViewportInstances[i]->UpdateSkeletalMeshAsync(true, true);

					PreviewCustomizableSkeletalComponent->SetCustomizableObjectInstance(ViewportInstances[i]);
					// TODO: UE-274335. We want this code to be able to operate with multiple components, not just the first one
					PreviewCustomizableSkeletalComponent->SetComponentName(CustomizableObject->GetComponentName(0));
					PreviewCustomizableSkeletalComponent->AttachToComponent(PreviewSkeletalMeshComponent, FAttachmentTransformRules::KeepRelativeTransform);

					ColliderComponent->AttachToComponent(PreviewSkeletalMeshComponent, FAttachmentTransformRules::KeepRelativeTransform);

					PreviewCustomizableSkeletalComponents.Add(PreviewCustomizableSkeletalComponent);
					PreviewSkeletalMeshComponents.Add(PreviewSkeletalMeshComponent);
					ColliderComponents.Add(ColliderComponent);
				}
			}
			
			// Adding components to viewport
			Viewport->SetPreviewComponents(PreviewSkeletalMeshComponents, ColliderComponents, ViewportColumns, InstanceSeparation);
		}
	}
}


bool FCustomizableObjectPopulationEditor::CanOpenInstance()
{
	if (Viewport.IsValid())
	{
		int32 InstanceIndex = Viewport->GetSelectedInstance();
		return  ViewportInstances.IsValidIndex(InstanceIndex);
	}

	return false;
}


void FCustomizableObjectPopulationEditor::OpenInstance()
{
	int32 SelectedInstance = Viewport->GetSelectedInstance();

	if (ViewportInstances[SelectedInstance])
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ViewportInstances[SelectedInstance]);
	}
}


void FCustomizableObjectPopulationEditor::OpenSkeletalMesh()
{
	int32 SelectedInstance = Viewport->GetSelectedInstance();

	if (!ViewportInstances[SelectedInstance])
	{
		return;
	}
	
	TMap<FName, TObjectPtr<USkeletalMesh>>::TConstIterator It = ViewportInstances[SelectedInstance]->GetPrivate()->SkeletalMeshes.CreateConstIterator();
	if (!It)
	{
		return;
	}
	
	USkeletalMesh* SkeletalMesh = It->Value;
	if (!SkeletalMesh)
	{
		return;
	}	

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(SkeletalMesh);
}

// To move elsewhere
class SSelectPopulationFolderDlg : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SSelectPopulationFolderDlg){}
		SLATE_ARGUMENT(FText, DefaultAssetPath)
		SLATE_ARGUMENT(FText, DefaultFileName)
	SLATE_END_ARGS()

		SSelectPopulationFolderDlg() : UserResponse(EAppReturnType::Cancel){}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

	/** FileName getter */
	FString GetFileName();

protected:
	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	void OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo);

	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText FileName;
};

void FCustomizableObjectPopulationEditor::GeneratePopulationInstances()
{
	if (!Population || !Population->IsValidPopulation())
	{
		UE_LOGF(LogMutable, Warning, "Unable to create customizable object instance assets from invalid population");
		return;
	}

	if (!Population->HasGenerator())
	{
		UE_LOGF(LogMutable, Warning, "Save the population before generating instances");
		return;
	}
	
	FString DefaultName = Population->Name;
	FText DefaultFileName = FText::Format(LOCTEXT("DefaultPopulationFileName", "{0}_Inst"),
		FText::FromString(DefaultName));

	TSharedRef<SSelectPopulationFolderDlg> FolderDlg =
		SNew(SSelectPopulationFolderDlg)
		.DefaultAssetPath(FText())
		.DefaultFileName(DefaultFileName);

	if (FolderDlg->ShowModal() != EAppReturnType::Cancel)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		Population->CompilePopulation();

		// overwrite window default value
		EAppReturnType::Type RetType = EAppReturnType::No;

		TArray<UCustomizableObjectInstance*> Instances;
		TArray<UCustomizableObjectInstance*> ExistingInstances;
		TArray<UObject*> ObjectsToSync;
		TArray<UPackage*> PackagesToSave;
		
		for (int32 i = 0; i < PopulationAssetInstancesNum; ++i)
		{
			FString ObjectName = FolderDlg->GetFileName();

			if (i > 0)
			{
				ObjectName += FString::Printf(TEXT("_%d"), i);
			}

			FString FilePath = FolderDlg->GetAssetPath() + FString("/") + ObjectName;
			FAssetData AssetData;

			if (RetType == EAppReturnType::No || RetType == EAppReturnType::Yes || RetType == EAppReturnType::YesAll)
			{
				//Search for asset with the same name
				FString ObjectPath = FilePath + FString(".") + ObjectName;
				AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));

				// Popup window
				if (AssetData.IsValid() && AssetData.GetClass() == UCustomizableObjectInstance::StaticClass() && RetType != EAppReturnType::YesAll)
				{
					FText Msg = FText::Format(LOCTEXT("FCustomizablePopulationEditorViewportClient_CreateInstance_Overwrite", "There is already an instance with name: '{0}'. Do you want to overwrite it?"),
						FText::FromString(ObjectName));
					RetType = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAllCancel, Msg);
				}
			}

			if (RetType == EAppReturnType::Cancel)
			{
				return;
			}

			UCustomizableObjectInstance* Instance = nullptr;

			if (RetType == EAppReturnType::No || RetType == EAppReturnType::NoAll || !AssetData.IsValid())
			{
				if (AssetData.IsValid() && i == 0)
				{
					FilePath += "_0";
				}

				FString PackageName;
				FString Name;

				// Determine an appropriate name
				FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
				AssetToolsModule.Get().CreateUniqueAssetName(FilePath, "", PackageName, Name);

				UPackage* Pkg = CreatePackage(*PackageName);
				PackagesToSave.Add(Pkg);
				Instance = NewObject<UCustomizableObjectInstance>(Pkg, FName(*Name), RF_Public | RF_Standalone);
				Pkg->MarkPackageDirty();
				ObjectsToSync.Add(Instance);
			}
			else
			{
				UCustomizableObjectInstance* ExistingInstance = Cast<UCustomizableObjectInstance>(AssetData.GetAsset());
				ExistingInstances.Add(ExistingInstance);

				Instance = NewObject<UCustomizableObjectInstance>(GetTransientPackage());
			}

			if (Instance)
			{
				Instances.Add(Instance);
			}
		}

		if (Instances.Num() > 0)
		{
			Population->GeneratePopulation(Instances, PopulationAssetInstancesNum);

			int32 ExistingInstancesIndx = 0;

			for (int32 i = 0; i < Instances.Num(); ++i)
			{
				// Update instances to overwrite
				if (Instances[i]->GetOutermost()->HasAnyFlags(EObjectFlags::RF_Transient))
				{
					ExistingInstances[ExistingInstancesIndx] = Instances[i]->Clone();
					PackagesToSave.Add(ExistingInstances[ExistingInstancesIndx]->GetPackage());
					ExistingInstancesIndx++;
				}
				else
				{
					//Notify that an asset has been created
					FAssetRegistryModule::AssetCreated(Instances[i]);
				}
			}

			if (ObjectsToSync.Num() > 0)
			{
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
			}

			FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, true);
		}
	}
}



/////////////////////////////////////////////////
// select folder dialog
//////////////////////////////////////////////////
void SSelectPopulationFolderDlg::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));
	FileName = InArgs._DefaultFileName;

	if (AssetPath.IsEmpty())
	{
		AssetPath = FText::FromString(TEXT("/Game"));
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SSelectPopulationFolderDlg::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("SSelectFolderDlg_Title", "Select target folder to store the instances"))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	//.SizingRule( ESizingRule::Autosized )
	.ClientSize(FVector2D(450, 450))
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot() // Add user input block
		.Padding(2)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectPath", "Select Path"))
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 14, "Regular"))
				]

				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(3)
				[
					ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FileName", "File Name"))
					.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 14, "Regular"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SEditableTextBox)
					.Text(InArgs._DefaultFileName)
					.OnTextCommitted(this, &SSelectPopulationFolderDlg::OnNameChange)
					.MinDesiredWidth(250)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			
			+ SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("OK", "OK"))
				.OnClicked(this, &SSelectPopulationFolderDlg::OnButtonClick, EAppReturnType::Ok)
			]
			
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SSelectPopulationFolderDlg::OnButtonClick, EAppReturnType::Cancel)
			]
		]
	]);
}

void SSelectPopulationFolderDlg::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
}

FReply SSelectPopulationFolderDlg::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	RequestDestroyWindow();

	return FReply::Handled();
}


void SSelectPopulationFolderDlg::OnNameChange(const FText& NewName, ETextCommit::Type CommitInfo)
{
	FileName = NewName;
}


EAppReturnType::Type SSelectPopulationFolderDlg::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SSelectPopulationFolderDlg::GetAssetPath()
{
	return AssetPath.ToString();
}


FString SSelectPopulationFolderDlg::GetFileName()
{
	return FileName.ToString();
}


#undef LOCTEXT_NAMESPACE
