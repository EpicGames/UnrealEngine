// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "MuCOPE/ICustomizableObjectPopulationEditor.h"
#include "TickableEditorObject.h"

class FReferenceCollector;
class FSpawnTabArgs;
class FUICommandList;
class IToolkitHost;
class SDockTab;
class SWidget;
class UCustomizableObjectInstance;
class UCustomizableObjectPopulation;
template <typename NumericType> class SNumericEntryBox;

class FCustomizableObjectPopulationEditor :
	public ICustomizableObjectPopulationEditor,
	public FEditorUndoClient,
	public FNotifyHook,
	public FTickableEditorObject,
	public FGCObject
{
public:

	FCustomizableObjectPopulationEditor();
	~FCustomizableObjectPopulationEditor();

	/** FGCObjet functions */
	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager) override;

	/** Editor init */
	void InitCustomizableObjectPopulationEditor(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulation* InObject);

	// IToolkit interface
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** FTickableGameObject interface */
	virtual bool IsTickable(void) const override;
	virtual void Tick(float InDeltaTime) override;
	virtual TStatId GetStatId() const override;

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	/** Begin ICustomAssetEditor initerface */
	virtual UCustomizableObjectPopulation* GetCustomAsset() { return Population; }
	virtual void SetCustomAsset(UCustomizableObjectPopulation* InCustomAsset) override;

	virtual FString GetReferencerName() const override
	{
		return TEXT("FCustomizableObjectPopulationEditor");
	}

private:

	/** Tab Spawners */
	TSharedRef<SDockTab> SpawnTab_PopulationProperties(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);

private:

	/** Adds the Customizable Object Population Editor commands to the default toolbar */
	void ExtendToolbar();

	/** Binds commands associated with the Static Mesh Editor. */
	void BindCommands();

	/** Save Customizable Object Population open in editor */
	void SaveAsset_Execute() override;
	bool CanOpenInstance();
	/** **/

	/** Inspect instances functions */
	void OpenInstance();
	void OpenSkeletalMesh();
	/** **/

	TSharedRef<SWidget> GenerateTestPopulationMenuContent(TSharedRef<FUICommandList> InCommandList);
	TOptional<int32> GetTestPopulationInstancesNum() const;
	void OnTestPopulationInstancesNumChanged(int32 Value);

	TSharedRef<SWidget> GeneratePopulationInstancesMenuContent(TSharedRef<FUICommandList> InCommandList);
	TOptional<int32> GetPopulationAssetInstancesNum() const;
	void OnPopulationAssetInstancesNumChanged(int32 Value);

	/** Generate instances in the viewport to test the population */
	void TestPopulation();

	/** Generates instance assets of the population */
	void GeneratePopulationInstances();

	/** Recompiles all the population generators of the population open in the editor */
	//void RecompileAllGenerators();

private:

	/** Pointer to the Population Class open in this editor */
	TObjectPtr<UCustomizableObjectPopulation> Population;

	/** Tabs IDs */
	static const FName PopulationPropertiesTabId;
	static const FName PopulationViewportTabId;

	/** Population Class Property View */
	TSharedPtr<class IDetailsView> PopulationDetailsView;

	/** Editor Viewport */
	TSharedPtr<class SCustomizableObjectPopulationEditorViewport> Viewport;

	// Number of instances that will be generated in the preview viewport
	int32 TestPopulationInstancesNum;

	// Number of instances that will be generated and saved
	int32 PopulationAssetInstancesNum;

	// Box to select the number of instances spawned in the viewport
	TSharedPtr< SNumericEntryBox<int32> > TestPopulationInstancesNumEntry;

	// Box to select the number of instances to save
	TSharedPtr< SNumericEntryBox<int32> > PopulationAssetInstancesNumEntry;

	//Viewport instance configuration
	int32 ViewportColumns;
	int32 InstanceSeparation;
	TSharedPtr< SNumericEntryBox<int32> > ViewportColumnsEntry;
	TSharedPtr< SNumericEntryBox<int32> > InstanceSeparationEntry;

	// Components to preview instances on the viewport
	TArray<class UCustomizableSkeletalComponent*> PreviewCustomizableSkeletalComponents;
	TArray<class USkeletalMeshComponent*> PreviewSkeletalMeshComponents;
	TArray<class UCapsuleComponent*> ColliderComponents;
	TArray<UCustomizableObjectInstance*> ViewportInstances;
	//
	
};
