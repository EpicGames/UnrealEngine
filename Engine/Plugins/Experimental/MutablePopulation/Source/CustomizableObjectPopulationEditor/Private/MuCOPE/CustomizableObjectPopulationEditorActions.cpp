// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationEditorActions.h"

#include "MuCOPE/CustomizableObjectPopulationEditorStyle.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationEditor"

FCustomizableObjectPopulationEditorCommands::FCustomizableObjectPopulationEditorCommands() : TCommands<FCustomizableObjectPopulationEditorCommands>
(
	"CustomizableObjectPopulationEditor", // Context name for fast lookup
	NSLOCTEXT("Contexts", "CustomizableObjectPopulationEditor", "CustomizableObject Population Editor"), // Localized context name for displaying
	NAME_None, // Parent
	FCustomizableObjectPopulationEditorStyle::GetStyleSetName()
	)
{

}

void FCustomizableObjectPopulationEditorCommands::RegisterCommands()
{
	UI_COMMAND(Save, "Save", "Save the Population.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TestPopulation, "Test Population", "Generate intances in the viewport to test the Population", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(GenerateInstances, "Generate Instances", "Generate random instances generated by this Population", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InspectSelectedInstance, "Inspect Instance", "Open the Customizable Object Instance Editor to inspect the selected instance in the preview viewport", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InspectSelectedSkeletalMesh, "Inspect Skeletal Mesh", "Open the skeletal mesh editor to inspect the selected instance in the preview viewport", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
