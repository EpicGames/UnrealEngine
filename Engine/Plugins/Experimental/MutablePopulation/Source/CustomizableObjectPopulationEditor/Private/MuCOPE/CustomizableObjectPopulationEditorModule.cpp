// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationEditorModule.h"

#include "MuCO/CustomizableObject.h"	// For the LogMutable log category
#include "MuCOPE/CustomizableObjectPopulationClassDetails.h"
#include "MuCOPE/CustomizableObjectPopulationClassEditor.h"
#include "MuCOPE/CustomizableObjectPopulationEditor.h"
#include "MuCOPE/CustomizableObjectPopulationEditorStyle.h"
#include "PropertyEditorModule.h"

class ICustomizableObjectPopulationClassEditor;
class ICustomizableObjectPopulationEditor;
class IToolkitHost;
class UCustomizableObjectPopulation;
class UCustomizableObjectPopulationClass;


const FName CustomizableObjectPopulationEditorAppIdentifier = FName(TEXT("CustomizableObjectPopulationEditorApp"));
const FName CustomizableObjectPopulationClassEditorAppIdentifier = FName(TEXT("CustomizableObjectPopulationClassEditorApp"));

#define LOCTEXT_NAMESPACE "MutableSettings"

/**
 * StaticMesh editor module
 */
class FCustomizableObjectPopulationEditorModule : public ICustomizableObjectPopulationEditorModule
{
public:

	// IModuleInterface interface
	void StartupModule() override;
	void ShutdownModule() override;

	// ICustomizableObjectEditorModule interface
	TSharedRef<ICustomizableObjectPopulationEditor> CreateCustomizableObjectPopulationEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulation* CustomizablePopulation) override;
	TSharedRef<ICustomizableObjectPopulationClassEditor> CreateCustomizableObjectPopulationClassEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulationClass* CustomizablePopulationClass) override;

	virtual TSharedPtr<FExtensibilityManager> GetCustomizableObjectPopulationEditorToolBarExtensibilityManager() override { return CustomizableObjectPopulationEditor_ToolBarExtensibilityManager; }

private:

	TSharedPtr<FExtensibilityManager> CustomizableObjectPopulationEditor_ToolBarExtensibilityManager;

};

IMPLEMENT_MODULE( FCustomizableObjectPopulationEditorModule, CustomizableObjectPopulationEditor );

static void LogWarning(const char* msg)
{
	// Can be called from any thread.
	UE_LOGF(LogMutable, Warning, "%ls", StringCast<TCHAR>(msg).Get() );
}


static void LogError(const char* msg)
{
	// Can be called from any thread.
	UE_LOGF(LogMutable, Error, "%ls", StringCast<TCHAR>(msg).Get());
}


static void* CustomMalloc(std::size_t Size_t, uint32_t Alignment)
{
	return FMemory::Malloc(Size_t, Alignment);
}


static void CustomFree(void* mem)
{
	return FMemory::Free(mem);
}


void FCustomizableObjectPopulationEditorModule::StartupModule()
{	
	// Property views
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	PropertyModule.RegisterCustomClassLayout("CustomizableObjectPopulationClass", FOnGetDetailCustomizationInstance::CreateStatic(&FCustomizableObjectPopulationClassDetails::MakeInstance));

	// Additional UI style
	FCustomizableObjectPopulationEditorStyle::Initialize();

	CustomizableObjectPopulationEditor_ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
}


void FCustomizableObjectPopulationEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("CustomizableObjectPopulationClass");
	}

	CustomizableObjectPopulationEditor_ToolBarExtensibilityManager.Reset();

	FCustomizableObjectPopulationEditorStyle::Shutdown();
}


TSharedRef<ICustomizableObjectPopulationEditor> FCustomizableObjectPopulationEditorModule::CreateCustomizableObjectPopulationEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulation* CustomizablePopulation)
{
	TSharedRef<FCustomizableObjectPopulationEditor> NewCustomizableObjectPopulationEditor(new FCustomizableObjectPopulationEditor());
	NewCustomizableObjectPopulationEditor->InitCustomizableObjectPopulationEditor(Mode, InitToolkitHost, CustomizablePopulation);
	return NewCustomizableObjectPopulationEditor;
}


TSharedRef<ICustomizableObjectPopulationClassEditor> FCustomizableObjectPopulationEditorModule::CreateCustomizableObjectPopulationClassEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulationClass* CustomizablePopulationClass)
{
	TSharedRef<FCustomizableObjectPopulationClassEditor> NewCustomizableObjectPopulationClassEditor(new FCustomizableObjectPopulationClassEditor());
	NewCustomizableObjectPopulationClassEditor->InitCustomizableObjectPopulationClassEditor(Mode, InitToolkitHost, CustomizablePopulationClass);
	return NewCustomizableObjectPopulationClassEditor;
}


#undef LOCTEXT_NAMESPACE
