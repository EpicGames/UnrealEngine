// Copyright Epic Games, Inc. All Rights Reserved.
#include "MuCOPE/CustomizableObjectPopulationEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

class ISlateStyle;

TSharedPtr<FSlateStyleSet> FCustomizableObjectPopulationEditorStyle::CustomizableObjectPopulationEditorStyleInstance = NULL;

	 
void FCustomizableObjectPopulationEditorStyle::Initialize()
{
	// Only register once
	if (!CustomizableObjectPopulationEditorStyleInstance.IsValid())
	{
		CustomizableObjectPopulationEditorStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*CustomizableObjectPopulationEditorStyleInstance);
	}
}


void FCustomizableObjectPopulationEditorStyle::Shutdown()
{
	if (CustomizableObjectPopulationEditorStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*CustomizableObjectPopulationEditorStyleInstance.Get());
		ensure(CustomizableObjectPopulationEditorStyleInstance.IsUnique());
		CustomizableObjectPopulationEditorStyleInstance.Reset();
	}
}


const ISlateStyle& FCustomizableObjectPopulationEditorStyle::Get()
{
	return *CustomizableObjectPopulationEditorStyleInstance;
}


FName FCustomizableObjectPopulationEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("CustomizableObjectPopulationEditorStyle"));
	return StyleSetName;
}


FString FCustomizableObjectPopulationEditorStyle::RelativePathToPluginPath(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("Mutable"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}


#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FCustomizableObjectPopulationEditorStyle::RelativePathToPluginPath( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_BRUSH(Style, RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( Style, RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

TSharedRef<FSlateStyleSet> FCustomizableObjectPopulationEditorStyle::Create()
{
	const FVector2D Icon20x20(20.0f, 20.0f);
	//const FVector2D Icon16x16(16.0f, 16.0f);
	//const FVector2D Icon8x8(8.0f, 8.0f);
	//const FVector2D Icon40x40(40.0f, 40.0f);
	//const FVector2D Icon64x64(64.0f, 64.0f);

	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("CustomizableObjectPopulationEditorStyle"));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	// Population Class editor
	Style->Set("CustomizableObjectPopulationClassEditor.SaveCustomizableObject", new IMAGE_BRUSH_SVG(Style, "Starship/Common/SaveCurrent", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.OpenCustomizableObjectEditor", new IMAGE_BRUSH_SVG(Style, "Starship/Common/blueprint", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.SaveEditorCurve", new IMAGE_BRUSH_SVG(Style, "Starship/Common/SaveCurrent", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.TestPopulationClass", new IMAGE_BRUSH_SVG(Style, "Starship/Common/play", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.GenerateInstances", new IMAGE_BRUSH_SVG(Style, "Starship/Persona/PersonaCreateAsset", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.InspectSelectedInstance", new IMAGE_BRUSH_SVG(Style, "Starship/Persona/SkeletalMesh", Icon20x20));
	Style->Set("CustomizableObjectPopulationClassEditor.InspectSelectedSkeletalMesh", new IMAGE_BRUSH_SVG(Style, "Starship/Common/SkeletalMesh", Icon20x20));

	// Population Editor
	Style->Set("CustomizableObjectPopulationEditor.TestPopulation", new IMAGE_BRUSH_SVG(Style, "Starship/Common/play", Icon20x20));
	Style->Set("CustomizableObjectPopulationEditor.GenerateInstances", new IMAGE_BRUSH_SVG(Style, "Starship/Persona/PersonaCreateAsset", Icon20x20));
	Style->Set("CustomizableObjectPopulationEditor.InspectSelectedInstance", new IMAGE_BRUSH_SVG(Style, "Starship/Persona/SkeletalMesh", Icon20x20));
	Style->Set("CustomizableObjectPopulationEditor.InspectSelectedSkeletalMesh", new IMAGE_BRUSH_SVG(Style, "Starship/Common/SkeletalMesh", Icon20x20));

	return Style;
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_BRUSH
#undef IMAGE_BRUSH_SVG
