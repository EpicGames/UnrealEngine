// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class ISlateStyle;

/** Manages the style which provides resources for niagara editor widgets. */
class FCustomizableObjectPopulationEditorStyle
{
public:

	// Creates and Registers the plugin SlateStyle
	static void Initialize();

	// Unregisters the plugin SlateStyle
	static void Shutdown();

	/** @return The Slate style set for niagara editor widgets */
	static const ISlateStyle& Get();

	static FName GetStyleSetName();
	static FString RelativePathToPluginPath(const FString& RelativePath, const ANSICHAR* Extension);


private:

	// Creates the plugin style
	static TSharedRef< class FSlateStyleSet > Create();

private:

	static TSharedPtr< class FSlateStyleSet > CustomizableObjectPopulationEditorStyleInstance;
};
