// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationEditorViewportClient.h"

#include "AdvancedPreviewScene.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "UnrealWidget.h"

class FSceneView;
class HHitProxy;

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationEditorViewportClient"

FCustomizableObjectPopulationEditorViewportClient::FCustomizableObjectPopulationEditorViewportClient(const TSharedRef<FAdvancedPreviewScene>& InPreviewScene)
	: FEditorViewportClient(nullptr, &InPreviewScene.Get())
{
	// Remove the initial gizmo
	Widget->SetDefaultVisibility(false);
	
	FAdvancedPreviewScene* PreviewSceneCasted = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	PreviewSceneCasted->SetEnvironmentVisibility(false);

	SelectedInstance = -1;
}


FCustomizableObjectPopulationEditorViewportClient::~FCustomizableObjectPopulationEditorViewportClient()
{
}


void FCustomizableObjectPopulationEditorViewportClient::Tick(float DeltaSeconds)
{
	FEditorViewportClient::Tick(DeltaSeconds);

	// Tick the preview scene world.
	PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaSeconds);
}


void FCustomizableObjectPopulationEditorViewportClient::SetPreviewComponent(TArray<USkeletalMeshComponent*> InSkeletalMeshComponent, TArray<UCapsuleComponent*> InColliderComponents)
{
	if (InSkeletalMeshComponent.Num() > 0)
	{
		// Removing the instances of the previous population
		SkeletalMeshComponents.Empty();
		ColliderComponents.Empty();

		// Cloning the new instances array
		SkeletalMeshComponents = ObjectPtrWrap(InSkeletalMeshComponent);
		ColliderComponents = ObjectPtrWrap(InColliderComponents);

		SelectedInstance = -1;

		// Hidding all the collider components
		for (int32 i = 0; i < ColliderComponents.Num(); ++i)
		{
			ColliderComponents[i]->SetVisibility(false);
			ColliderComponents[i]->ShapeColor = FColor(0,255,0,255);
		}
	}
}


void FCustomizableObjectPopulationEditorViewportClient::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (int32 i = 0; i < SkeletalMeshComponents.Num(); ++i)
	{
		Collector.AddReferencedObject(SkeletalMeshComponents[i]);
		Collector.AddReferencedObject(ColliderComponents[i]);
	}
}


void FCustomizableObjectPopulationEditorViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	// Creating ray from mouse click
	FViewportCursorLocation MouseViewportRay(&View, this, HitX, HitY);
	FHitResult HitResult(1.0f);
	const FVector RayInit(MouseViewportRay.GetOrigin());
	const FVector RayEnd(MouseViewportRay.GetOrigin() + MouseViewportRay.GetDirection()*10000.0f);

	FVector ViewLocation = GetViewLocation();
	FVector SelectionHit(HALF_WORLD_MAX);
	SelectedInstance = -1;

	// Checking hit
	for (int32 i = 0; i < ColliderComponents.Num(); ++i)
	{
		if (SkeletalMeshComponents[i] && SkeletalMeshComponents[i]->GetSkinnedAsset())
		{	
			FVector BoxExtent = SkeletalMeshComponents[i]->Bounds.BoxExtent;
			ColliderComponents[i]->SetVisibility(false);
			
			// Setting the collider to an aproximation of the skeleta mesh BB
			if (BoxExtent.Z != ColliderComponents[i]->GetUnscaledCapsuleHalfHeight())
			{
				FVector Location = ColliderComponents[i]->GetRelativeLocation();
				Location.Z += BoxExtent.Z;

				ColliderComponents[i]->SetRelativeLocation(Location);
				ColliderComponents[i]->SetCapsuleSize(BoxExtent.X > BoxExtent.Y ? BoxExtent.Y : BoxExtent.X, BoxExtent.Z);
			}

			//Cheking if the collider hits any collider capsule
			if (ColliderComponents[i]->LineTraceComponent(HitResult, RayInit, RayEnd, FCollisionQueryParams(FName(TEXT("trace")), true)))
			{
				if (FVector::DistSquared(ViewLocation, HitResult.Location) < FVector::DistSquared(ViewLocation, SelectionHit))
				{
					SelectionHit = HitResult.Location;
					SelectedInstance = i; 
				}
			}
		}
	}

	if (SelectedInstance != -1)
	{
		ColliderComponents[SelectedInstance]->SetVisibility(true);
	}
}


#undef LOCTEXT_NAMESPACE
