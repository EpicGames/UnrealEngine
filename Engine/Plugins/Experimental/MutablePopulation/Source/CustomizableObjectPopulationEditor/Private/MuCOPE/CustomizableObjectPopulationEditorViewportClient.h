// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

class FAdvancedPreviewScene;
class FReferenceCollector;
class FSceneView;
class HHitProxy;
class USkeletalMeshComponent;


class FCustomizableObjectPopulationEditorViewportClient : public FEditorViewportClient
{
public:

	FCustomizableObjectPopulationEditorViewportClient(const TSharedRef<FAdvancedPreviewScene>& InPreviewScene);
	~FCustomizableObjectPopulationEditorViewportClient();

	virtual void Tick(float DeltaSeconds) override;

	void SetPreviewComponent(TArray<USkeletalMeshComponent*> InSkeletalMeshComponent, TArray<class UCapsuleComponent*> InColliderComponents);

	// FSerializableObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FSerializableObject interface

	// Select an instance of the viewport
	void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;

	int32 GetSelectedInstance() { return SelectedInstance; }

private:

	// All the Skeletal Mesh components of the scene
	TArray<TObjectPtr<USkeletalMeshComponent>> SkeletalMeshComponents;

	// All the Collider components of the scene
	TArray<TObjectPtr<class UCapsuleComponent>> ColliderComponents;

	// index of the selected instance
	int32 SelectedInstance;

};
