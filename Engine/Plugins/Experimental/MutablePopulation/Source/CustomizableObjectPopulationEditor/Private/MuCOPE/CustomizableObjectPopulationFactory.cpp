// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/CustomizableObjectPopulationFactory.h"

#include "MuCOP/CustomizableObjectPopulation.h"

class FFeedbackContext;
class UClass;
class UObject;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationFactory"

UCustomizableObjectPopulationFactory::UCustomizableObjectPopulationFactory() : Super()
{
	// Property initialization
	bCreateNew = true;
	SupportedClass = UCustomizableObjectPopulation::StaticClass();
	bEditAfterNew = true;
}

 UObject* UCustomizableObjectPopulationFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	 UCustomizableObjectPopulation* CustomizableObjectPopulation = NewObject<UCustomizableObjectPopulation>(InParent, Class, Name, Flags);
	 
	 return CustomizableObjectPopulation;
}

 bool UCustomizableObjectPopulationFactory::ShouldShowInNewMenu() const
 {
	 return true;
 }

#undef LOCTEXT_NAMESPACE
