// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/SCustomizableObjectPopulationClassTagManager.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClassTagsTool"

void SCustomizableObjectPopulationClassTagsManager::Construct(const FArguments& InArgs)
{
	CustomizableObject = InArgs._CustomizableObject;
	RootObject = InArgs._RootObject;
	
	RebuildWidget();	
}

void SCustomizableObjectPopulationClassTagsManager::RebuildWidget()
{
	// Creating the Tag widgets
	BuildTagSelector();
	BuildTagManager();

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar).AlwaysShowScrollbar(true);

	// Getting the scroll position before update
	float OldScrollOffset = 0.0f;
	if (ScrollBox.IsValid())
	{
		OldScrollOffset = ScrollBox->GetScrollOffset();
	}

	ChildSlot
		[
			SAssignNew(ScrollBox, SScrollBox)
			.ExternalScrollbar(ScrollBar)
			.ScrollBarAlwaysVisible(true)
			+ SScrollBox::Slot()
			[
	
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 10.0f, 0.0f, 0.0f)
				[
					TagSelector.ToSharedRef()
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f,10.0f,0.0f,0.0f)
				[
					TagManager.ToSharedRef()
				]
			]
	];

	// Setting the old scroll position after update
	if (ScrollBox.IsValid())
	{
		ScrollBox->SetScrollOffset(OldScrollOffset);
	}
}

void SCustomizableObjectPopulationClassTagsManager::BuildTagSelector()
{
	TagOptions.Empty();

	TagSelector = SNew(SVerticalBox);

	TagSelector->AddSlot()
	.AutoHeight()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("TagSelector","Tag Selector:"))
		.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
	];

	TagSelector->AddSlot()
	.AutoHeight()
	.Padding(5.0f,10.0f,0.0f,0.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%d Tags"),CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags().Num())))
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.OnClicked(this, &SCustomizableObjectPopulationClassTagsManager::OnAddTagSelectorButtonPressed)
			.ToolTipText(LOCTEXT("AddTagSelector", "Add a Tag to the Customizable Object"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Plus")))
			]
		]
	];

	for (int32 i = 0; i < RootObject->GetPrivate()->GetPopulationClassTags().Num(); ++i)
	{
		TagOptions.Add(MakeShareable(new FString(RootObject->GetPrivate()->GetPopulationClassTags()[i])));
	}

	TagOptions.Add(MakeShareable(new FString("None")));

	for (int32 i = 0; i < CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags().Num(); ++i)
	{
		int32 TagIndex = TagOptions.Num()-1;
		TSharedPtr<FString > SelectedOption;

		for (int32 j = 0; j < RootObject->GetPrivate()->GetPopulationClassTags().Num(); ++j)
		{
			if (CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags()[i] == RootObject->GetPrivate()->GetPopulationClassTags()[j])
			{
				TagIndex = j;
			}
		}

		SelectedOption = TagOptions[TagIndex];

		TagSelector->AddSlot()
		.AutoHeight()
		.Padding(15.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextComboBox)
				.OptionsSource(&TagOptions)
				.InitiallySelectedItem(SelectedOption)
				.OnSelectionChanged(this,&SCustomizableObjectPopulationClassTagsManager::OnComboBoxSelectionChanged, i)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SCustomizableObjectPopulationClassTagsManager::OnRemoveTagSelectorClicked, i)
				.ToolTipText(LOCTEXT("RemoveTag", "Remove Tag"))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]
		];
	}
}


FReply SCustomizableObjectPopulationClassTagsManager::OnAddTagSelectorButtonPressed()
{
	CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags().Add("");
	CustomizableObject->MarkPackageDirty();

	RebuildWidget();

	return FReply::Handled();
}


void SCustomizableObjectPopulationClassTagsManager::OnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, int32 Index)
{
	if (Selection.IsValid())
	{
		CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags()[Index] = *Selection;
		CustomizableObject->MarkPackageDirty();
	}
}

FReply SCustomizableObjectPopulationClassTagsManager::OnRemoveTagSelectorClicked(int32 Index)
{
	CustomizableObject->GetPrivate()->GetCustomizableObjectClassTags().RemoveAt(Index);
	CustomizableObject->MarkPackageDirty();

	RebuildWidget();

	return FReply::Handled();
}


void SCustomizableObjectPopulationClassTagsManager::BuildTagManager()
{
	TagManager = SNew(SVerticalBox);

	TagManager->AddSlot()
	.AutoHeight()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("TagManager", "Tag Manager:"))
		.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
	];

	TagManager->AddSlot()
	.AutoHeight()
	.Padding(5.0f, 10.0f, 0.0f, 0.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("%d Tags"), RootObject->GetPrivate()->GetPopulationClassTags().Num())))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.OnClicked(this, &SCustomizableObjectPopulationClassTagsManager::OnAddTagButtonPressed)
			.ToolTipText(LOCTEXT("AddTag", "Add a Tag to the Root Object"))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(TEXT("Plus")))
			]
		]
	];

	for (int32 i = 0; i < RootObject->GetPrivate()->GetPopulationClassTags().Num(); ++i)
	{
		TagManager->AddSlot()
		.AutoHeight()
		.Padding(15.0f, 10.0f, 0.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 3.0f, 0.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(FString::Printf(TEXT("%d."), i+1)))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SEditableTextBox)
				.Text(FText::FromString(RootObject->GetPrivate()->GetPopulationClassTags()[i]))
				.OnTextCommitted(this, &SCustomizableObjectPopulationClassTagsManager::OnTextCommited, i)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.OnClicked(this, &SCustomizableObjectPopulationClassTagsManager::OnRemoveTagClicked, i)
				.ToolTipText(LOCTEXT("RemoveTag", "Remove Tag"))
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]
		];
	}
}


FReply SCustomizableObjectPopulationClassTagsManager::OnAddTagButtonPressed()
{
	RootObject->GetPrivate()->GetPopulationClassTags().Add("");
	RootObject->MarkPackageDirty();

	RebuildWidget();

	return FReply::Handled();
}


FReply SCustomizableObjectPopulationClassTagsManager::OnRemoveTagClicked(int32 Index)
{
	RootObject->GetPrivate()->GetPopulationClassTags().RemoveAt(Index);
	RootObject->MarkPackageDirty();

	RebuildWidget();

	return FReply::Handled();
}


void SCustomizableObjectPopulationClassTagsManager::OnTextCommited(const FText& NewText, ETextCommit::Type InTextCommit, int32 Index)
{
	if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
	{
		RootObject->GetPrivate()->GetPopulationClassTags()[Index] = NewText.ToString();
		RootObject->MarkPackageDirty();

		RebuildWidget();
	}
}

#undef LOCTEXT_NAMESPACE
