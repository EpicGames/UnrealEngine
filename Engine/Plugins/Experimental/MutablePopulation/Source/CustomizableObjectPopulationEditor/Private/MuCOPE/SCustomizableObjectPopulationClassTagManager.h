// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FString;
class FText;
class SVerticalBox;
class UCustomizableObject;

//Tool to manage the population class tags of a customizable object
class SCustomizableObjectPopulationClassTagsManager : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectPopulationClassTagsManager) {}
	SLATE_ARGUMENT(UCustomizableObject*, CustomizableObject)
	SLATE_ARGUMENT(UCustomizableObject*, RootObject)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void RebuildWidget();

	void BuildTagSelector();
	void BuildTagManager();

	FReply OnAddTagSelectorButtonPressed();
	FReply OnAddTagButtonPressed();
	void OnComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo, int32 Index);
	FReply OnRemoveTagSelectorClicked(int32 Index);
	FReply OnRemoveTagClicked(int32 Index);
	void OnTextCommited(const FText& NewText, ETextCommit::Type InTextCommit, int32 Index);

private:

	/** Pointer to the Customizable object opened in the editor*/
	UCustomizableObject* CustomizableObject;

	/** Pointer to the root Customizable Object */
	UCustomizableObject* RootObject;

	/** Widget Scrollbox */
	TSharedPtr<class SScrollBox> ScrollBox;

	/**  */
	TSharedPtr<SVerticalBox> TagSelector;

	/**  */
	TSharedPtr<SVerticalBox> TagManager;

	/**  */
	TArray<TSharedPtr<FString>> TagOptions;

};
