// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/SCustomizableObjectPopulationClassTagsTool.h"

#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectPrivate.h"
#include "MuCOP/CustomizableObjectPopulationClass.h"
#include "MuCOPE/CustomizableObjectPopulationClassDetails.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"

struct FGeometry;
struct FSlateBrush;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationClassTagsTool"


void SCustomizableObjectPopulationClassTagsTool::Construct(const FArguments & InArgs)
{
	PopulationClass = InArgs._PopulationClass;

	GenerateChildSlot();
}


void SCustomizableObjectPopulationClassTagsTool::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Need to be updated if...
	if (PopulationClass)
	{
		// The Customizable Object has been selected
		if (bNeedsUpdate && PopulationClass->CustomizableObject)
		{
			GenerateChildSlot();
		}

		// The Customizable Object has been unselected
		if (!bNeedsUpdate && !PopulationClass->CustomizableObject)
		{
			GenerateChildSlot();
		}

		// The Customizable Object has changed
		if (SelectedCustomizableObject != PopulationClass->CustomizableObject)
		{
			GenerateChildSlot();
		}
	}
}


void SCustomizableObjectPopulationClassTagsTool::GenerateChildSlot()
{
	if (PopulationClass->CustomizableObject)
	{
		bNeedsUpdate = false;
		SelectedCustomizableObject = PopulationClass->CustomizableObject;

		float OldScrollOffset = 0.0f;
		if (ScrollBox.IsValid())
		{
			OldScrollOffset = ScrollBox->GetScrollOffset();
		}

		TagsViewer = SNew(SVerticalBox);

		// Generating widgets
		GenerateTagsManager();
		GenerateTagsViewer();
		GenerateParameterManager();
		GenerateParameterOptionsManager();

		TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar).AlwaysShowScrollbar(true).Thickness(FVector2D(12.0f, 12.0f));
		TSharedRef<SScrollBar> TagsScrollBar = SNew(SScrollBar).AlwaysShowScrollbar(false).Thickness(FVector2D(12.0f, 12.0f));

		ChildSlot
		[
			SAssignNew(ScrollBox,SScrollBox)
			.ExternalScrollbar(ScrollBar)
			.ScrollBarAlwaysVisible(true)

			+ SScrollBox::Slot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(TagsManagerExpandableArea, SExpandableArea)
					.InitiallyCollapsed(false)
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					.BorderImage_Lambda([this]() { return GetExpandableAreaBorderImage(*TagsManagerExpandableArea); })
					.BodyBorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderBackgroundColor(FLinearColor::White)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Margin(FMargin(0.0f,5.0f))
						.Text(LOCTEXT("ClassTagManagementText", "Population Tag Management for the current Customizable Object"))
						.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
					.BodyContent()
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							TagsManager.ToSharedRef()
						]
						
						+ SVerticalBox::Slot()
						.AutoHeight()
						.MaxHeight(150.0f)
						.Padding(10.0f, 10.0f, 0.0f, 0.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.0)
							[
								SAssignNew(TagsScrollBox, SScrollBox)
								.ExternalScrollbar(TagsScrollBar)
								.ScrollBarAlwaysVisible(false)
								+ SScrollBox::Slot()
								[
									TagsViewer.ToSharedRef()
								]
							]
							
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBox)
								.WidthOverride(FOptionalSize(16))
								[
									TagsScrollBar
								]
							]
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f,10.0f,0.0f,0.0f)
				[
					SAssignNew(ParameterOptionsManagerExpandableArea, SExpandableArea)
					.InitiallyCollapsed(false)
					.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
					.BorderImage_Lambda([this]() { return GetExpandableAreaBorderImage(*ParameterOptionsManagerExpandableArea); })
					.BodyBorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderBackgroundColor(FLinearColor::White)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Margin(FMargin(0.0f, 5.0f))
						.Text(LOCTEXT("ParameterOptionsManagerText", "Parameter Option Population Tags"))
						.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
						.ShadowOffset(FVector2D(1.0f, 1.0f))
					]
					.BodyContent()
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(5.0f, 10.0f, 0.0f, 0.0f)
						[
							ParametersManager.ToSharedRef()
						]
						
						+ SVerticalBox::Slot()
						.Padding(5.0f, 10.0f, 0.0f, 0.0f)
						.AutoHeight()
						[
							ParameterOptionsManager.ToSharedRef()
						]
					]
				]
			]
		];

		if (ScrollBox.IsValid())
		{
			ScrollBox->SetScrollOffset(OldScrollOffset);
		}
	}
	else
	{
		bNeedsUpdate = true;

		ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(10.0f, 10.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Warning Message", "Select a Customizable object for this Population class"))
				]
			];
	}
}


void SCustomizableObjectPopulationClassTagsTool::GenerateTagsManager()
{
	TagsManager = SNew(SVerticalBox);

	TagsManager->AddSlot()
	.Padding(5.0f, 10.0f, 0.0f, 0.0f)
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(3.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(SearchBoxWidget, SSearchBox)
			.HintText(LOCTEXT("HintTextTag","Search/New Tag..."))
			.OnTextCommitted(this, &SCustomizableObjectPopulationClassTagsTool::OnSearchBoxTextCommitted)
			.OnTextChanged(this, &SCustomizableObjectPopulationClassTagsTool::OnSearchBoxFilterTextChanged)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(10.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SButton)
			.ToolTipText(FText::FromString("Adds a new tag to the Customizable Object of the Population Class"))
			.OnClicked(this, &SCustomizableObjectPopulationClassTagsTool::OnAddTagButtonPressed)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AddTagText","+ Add Tag"))
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
			]
		]
	];
}


void SCustomizableObjectPopulationClassTagsTool::GenerateTagsViewer()
{
	TagsViewer->ClearChildren();

	for (int32 i = 0; i < PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Num(); ++i)
	{
		FString TagValue = PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags()[i];

		if (SearchItem.IsEmpty())
		{
			TagsViewer->AddSlot()
			.AutoHeight()
			.Padding(0.0f,7.0f,0.0f,0.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f,2.0f,5.0f,0.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString("- " + TagValue))
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.ToolTipText(FText::FromString("Delete the selected tag in the CommboBox"))
					.OnClicked(this, &SCustomizableObjectPopulationClassTagsTool::OnRemoveTagButtonPressed,i)
					.ButtonColorAndOpacity(FSlateColor(FLinearColor(1.0f,0.2f,0.2f)))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(TEXT("Cross")))
					]
				]
			];
		}
		else
		{
			if (TagValue.Contains(SearchItem))
			{
				TagsViewer->AddSlot()
				.AutoHeight()
				.Padding(0.0f, 7.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.0f, 2.0f, 5.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString("- "+TagValue))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.ToolTipText(FText::FromString("Delete the selected tag in the CommboBox"))
						.OnClicked(this, &SCustomizableObjectPopulationClassTagsTool::OnRemoveTagButtonPressed,i)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush(TEXT("Cross")))
						]
					]
				];
			}
		}
	}
}


void SCustomizableObjectPopulationClassTagsTool::GenerateParameterManager()
{
	ParametersManager = SNew(SHorizontalBox);
	
	ParametersManager->AddSlot()
	.AutoWidth()
	.Padding(5.0f, 5.0f, 0.0f, 0.0f)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 3.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ParameterText","Parameter Selected:"))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(ParameterComboBox, STextComboBox)
			.OptionsSource(&ParameterComboBoxOptions)
			.OnSelectionChanged(this, &SCustomizableObjectPopulationClassTagsTool::OnParameterComboBoxSelectionChanged)
		]

		//+ SVerticalBox::Slot()
		//.AutoHeight()
		//[
		//	SNew(SHorizontalBox)
		//	+SHorizontalBox::Slot()
		//	.AutoWidth()
		//	[
		//		SNew(SButton)
		//		.Text(LOCTEXT("AddTagButtonText","Add Tag"))
		//		.OnClicked(this, &SCustomizableObjectPopulationClassTagsTool::OnParameterAddTagButtonPressed)
		//	]
		//]
	];

	// Fill Parameters Combobox source
	ParameterComboBoxOptions.Empty();
	for (int32 i = 0; i < PopulationClass->CustomizableObject->GetParameterCount(); ++i)
	{
		ParameterComboBoxOptions.Add(MakeShareable(new FString(PopulationClass->CustomizableObject->GetParameterName(i))));
		if (LastSelectedParameter == PopulationClass->CustomizableObject->GetParameterName(i))
		{
			ParameterComboBox->SetSelectedItem(ParameterComboBoxOptions.Last());
		}
	}

	// Sort Parameters by name
	ParameterComboBoxOptions.Sort
	(
		[](const TSharedPtr<FString> A, const TSharedPtr<FString> B)
	{
		return (*A).Compare(*B) < 0;
	}
	);

	// TODO: Uncomment this section when parameters can have tags
	//if (ParameterComboBox->GetSelectedItem().IsValid() && PopulationClass->CustomizableObject->CustomizableObjectParametersTags.Find(*ParameterComboBox->GetSelectedItem()))
	//{
	//	FString SelectedParameter = *ParameterComboBox->GetSelectedItem();
	//	
	//	float LineLength = 40.0f;
	//
	//	LineLength += 40.0f * (int)((PopulationClass->CustomizableObject->CustomizableObjectParametersTags[SelectedParameter].Tags.Num() - 1) / 4);
	//	
	//	ParametersManager->AddSlot()
	//	.AutoWidth()
	//	.Padding(20.0f, 5.0f, 0.0f, 0.0f)
	//	[
	//		SNew(SCustomEditorLine)
	//		.Length(LineLength)
	//		.Horizontal(false)
	//	];
	//
	//	// Delete the parameter from the tags map if it doesn't have any tag and any parameter option with a tag
	//	if (PopulationClass->CustomizableObject->CustomizableObjectParametersTags[SelectedParameter].Tags.Num() == 0
	//		&& PopulationClass->CustomizableObject->CustomizableObjectParametersTags[SelectedParameter].ParameterOptions.Num()==0)
	//	{
	//		PopulationClass->CustomizableObject->CustomizableObjectParametersTags.Remove(SelectedParameter);
	//	}
	//	else
	//	{
	//		ParametersManager->AddSlot()
	//		.AutoWidth()
	//		.Padding(20.0f, 5.0f, 0.0f, 0.0f)
	//		[
	//			SNew(SParameterTagWidget)
	//			.PopulationClass(PopulationClass)
	//			.TagsList(&PopulationClass->CustomizableObject->CustomizableObjectParametersTags[SelectedParameter].Tags)
	//			.TagsToolPtr(this)
	//		];
	//	}
	//}
	
}


void SCustomizableObjectPopulationClassTagsTool::GenerateParameterOptionsManager()
{
	ParameterOptionsManager.Reset();
	ParameterOptionsManager = SNew(SVerticalBox);

	if (ParameterComboBox->GetSelectedItem().IsValid())
	{
		FString SelectedParameter = *ParameterComboBox->GetSelectedItem();

		ParameterOptionsManager->AddSlot()
		.AutoHeight()
		.Padding(5.0f, 5.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ParameterOptionsTextTitle", "Parameter Options:"))
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
		];


		for (int32 i = 0; i < PopulationClass->CustomizableObject->GetParameterCount(); ++i)
		{
			if (SelectedParameter == PopulationClass->CustomizableObject->GetParameterName(i))
			{
				if (PopulationClass->CustomizableObject->GetPrivate()->GetEnumParameterNumValues(i) > 0)
				{
					for (int32 j = 0; j < PopulationClass->CustomizableObject->GetPrivate()->GetEnumParameterNumValues(i); ++j)
					{
						TSharedPtr<SHorizontalBox> VerticalWidget = SNew(SHorizontalBox);
						FString ParameterOptionName = PopulationClass->CustomizableObject->GetPrivate()->GetIntParameterAvailableOption(i, j);

						VerticalWidget->AddSlot()
						.AutoWidth()
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.Padding(5.0f, 0.0f, 0.0f, 0.0f)
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(FText::FromString(ParameterOptionName + ":"))
							]

							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(5.0f,0.0f,0.0f,0.0f)
							[
								SNew(SButton)
								.Text(LOCTEXT("AddTagButtonText", "Add Tag"))
								.OnClicked(this, &SCustomizableObjectPopulationClassTagsTool::OnParameterOptionAddTagButtonPressed, ParameterOptionName)
							]
						];

						if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Find(SelectedParameter))
						{
							if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions.Find(ParameterOptionName))
							{
								float LineLength = 40.0f;

								LineLength += 40.0f * (int)((PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions[ParameterOptionName].Tags.Num() - 1) / 4);
								
								VerticalWidget->AddSlot()
								.AutoWidth()
								.Padding(20.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(SCustomEditorLine)
									.Horizontal(false)
									.Length(LineLength)
								];

								// Delete the parameter option from the parameter options map if it doesn't have any tag
								if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions[ParameterOptionName].Tags.Num() == 0)
								{
									PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions.Remove(ParameterOptionName);

									// Check if we also have to delete the parameter 
									if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].Tags.Num() == 0
										&& PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions.Num() == 0)
									{
										PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Remove(SelectedParameter);
									}
								}
								else
								{
									VerticalWidget->AddSlot()
									.AutoWidth()
									.Padding(20.0f, 0.0f, 0.0f, 0.0f)
									[
										SNew(SParameterTagWidget)
										.PopulationClass(PopulationClass)
										.TagsList(&PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions[ParameterOptionName].Tags)
										.TagsToolPtr(this)
									];
								}
							}
						}

						ParameterOptionsManager->AddSlot()
						.AutoHeight()
						.Padding(5.0f, 10.0f, 0.0f, 30.0f)
						[
							VerticalWidget.ToSharedRef()
						];
					}
				}
				else
				{
					ParameterOptionsManager->AddSlot()
					.AutoHeight()
					.Padding(5.0f, 10.0f, 0.0f, 0.0f)
					[
						SNew(STextBlock)
						.Text(FText::FromString(FString::Printf(TEXT("The parameter %s doesn't have any option"),*SelectedParameter)))
					];
				}

				break;
			}
		}			
	}
}


FReply SCustomizableObjectPopulationClassTagsTool::OnAddTagButtonPressed()
{
	FString NewTag = SearchBoxWidget->GetText().ToString();

	if (!NewTag.IsEmpty() && PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Find(NewTag) == INDEX_NONE)
	{
		PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Add(NewTag);
		PopulationClass->CustomizableObject->MarkPackageDirty();
		SearchItem.Empty();
		GenerateChildSlot();
	}

	return FReply::Handled();
}


FReply SCustomizableObjectPopulationClassTagsTool::OnRemoveTagButtonPressed(int32 Index)
{
	PopulationClass->CustomizableObject->MarkPackageDirty();
	PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().RemoveAt(Index);
	GenerateChildSlot();

	return FReply::Unhandled();
}


void SCustomizableObjectPopulationClassTagsTool::OnSearchBoxFilterTextChanged(const FText& InText)
{
	SearchItem = InText.ToString();
	GenerateTagsViewer();
}


void SCustomizableObjectPopulationClassTagsTool::OnSearchBoxTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		FString NewTag = InText.ToString();

		// If the written tag doesn't exist, then we create a new one
		if (!NewTag.IsEmpty() && PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Find(NewTag) == INDEX_NONE)
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Add(InText.ToString());
			PopulationClass->CustomizableObject->MarkPackageDirty();
			SearchItem.Empty();
			GenerateChildSlot();
		}
	}
}


void SCustomizableObjectPopulationClassTagsTool::OnParameterComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid() && SelectInfo == ESelectInfo::OnMouseClick)
	{
		ParameterComboBox->SetSelectedItem(Selection);
		LastSelectedParameter = *Selection;
		
		GenerateChildSlot();
	}
}


FReply SCustomizableObjectPopulationClassTagsTool::OnParameterAddTagButtonPressed()
{
	if (ParameterComboBox->GetSelectedItem().IsValid())
	{
		FString SelectedParameter = *ParameterComboBox->GetSelectedItem();

		if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Find(SelectedParameter))
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].Tags.Add("");
			GenerateChildSlot();
		}
		else
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Add(SelectedParameter);
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].Tags.Add("");
			GenerateChildSlot();
		}

		PopulationClass->MarkPackageDirty();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FReply SCustomizableObjectPopulationClassTagsTool::OnParameterOptionAddTagButtonPressed(FString ParameterOptionName)
{
	if(ParameterComboBox->GetSelectedItem().IsValid())
	{
		FString SelectedParameter = *ParameterComboBox->GetSelectedItem();

		if (!PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Find(SelectedParameter))
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags().Add(SelectedParameter);
		}

		if (PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions.Find(ParameterOptionName))
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions[ParameterOptionName].Tags.Add("");
			GenerateChildSlot();
		}
		else
		{
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions.Add(ParameterOptionName);
			PopulationClass->CustomizableObject->GetPrivate()->GetCustomizableObjectParametersTags()[SelectedParameter].ParameterOptions[ParameterOptionName].Tags.Add("");
			GenerateChildSlot();
		}

		PopulationClass->CustomizableObject->MarkPackageDirty();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

const FSlateBrush* SCustomizableObjectPopulationClassTagsTool::GetExpandableAreaBorderImage(const SExpandableArea& Area)
{
	if (Area.IsTitleHovered())
	{
		return Area.IsExpanded() ? FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered") : FAppStyle::GetBrush("DetailsView.CollapsedCategory_Hovered");
	}
	return Area.IsExpanded() ? FAppStyle::GetBrush("DetailsView.CategoryTop") : FAppStyle::GetBrush("DetailsView.CollapsedCategory");
}


// Parameter Widget -------------------------

void SParameterTagWidget::Construct(const FArguments & InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	TagsList = InArgs._TagsList;
	TagsToolPtr = InArgs._TagsToolPtr;

	Rows = SNew(SVerticalBox);

	for (int32 i = 0; i < TagsList->Num(); i+=4)
	{
		TSharedPtr< SHorizontalBox> Row = SNew(SHorizontalBox);
		
		for (int32 j = i; j < i+4 && j < TagsList->Num(); ++j)
		{
			Row->AddSlot()
			.AutoWidth()
			[
				SNew(SSingleTagWidget)
				.PopulationClass(PopulationClass)
				.TagIndex(j)
				.TagsList(TagsList)
				.TagsToolPtr(TagsToolPtr)
			];
		}

		Rows->AddSlot()
		.AutoHeight()
		.Padding(5.0f,0.0f,0.0f,0.0f)
		[
			Row.ToSharedRef()
		];
	}
	
	ChildSlot
	[
		Rows.ToSharedRef()
	];
}


// Single Tag widget -------------------------

void SSingleTagWidget::Construct(const FArguments& InArgs)
{
	PopulationClass = InArgs._PopulationClass;
	TagIndex = InArgs._TagIndex;
	TagsList = InArgs._TagsList;
	TagsToolPtr = InArgs._TagsToolPtr;
	
	ChildSlot
	.Padding(0.0f,0.0f,10.0f,0.0f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("Tag %d:"), TagIndex)))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(TagsComboBox, STextComboBox)
				.OptionsSource(&ComboBoxTagsOptions)
				.OnSelectionChanged(this, &SSingleTagWidget::OnTagsComboBoxSelectionChanged)
			]
			
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(FText::FromString("Delete the selected tag in the CommboBox"))
				.OnClicked(this, &SSingleTagWidget::OnRemoveTagButtonPressed)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
				]
			]
		]
	];

	ComboBoxTagsOptions.Empty();
	ComboBoxTagsOptions.Add(MakeShareable(new FString("None")));
	TagsComboBox->SetSelectedItem(ComboBoxTagsOptions.Last());

	for (int32 i = 0; i < PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags().Num(); ++i)
	{
		ComboBoxTagsOptions.Add(MakeShareable(new FString(PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags()[i])));

		if ((*TagsList)[TagIndex] == PopulationClass->CustomizableObject->GetPrivate()->GetPopulationClassTags()[i])
		{
			TagsComboBox->SetSelectedItem(ComboBoxTagsOptions.Last());
		}
	}

}


void SSingleTagWidget::OnTagsComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo)
{
	if (Selection.IsValid() && SelectInfo == ESelectInfo::OnMouseClick)
	{
		TagsComboBox->SetSelectedItem(Selection);
		(*TagsList)[TagIndex] = *Selection;
		PopulationClass->CustomizableObject->MarkPackageDirty();
	}
}


FReply SSingleTagWidget::OnRemoveTagButtonPressed()
{
	TagsList->RemoveAt(TagIndex);
	TagsToolPtr->GenerateChildSlot();
	PopulationClass->CustomizableObject->MarkPackageDirty();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
