// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }

class FText;
class SExpandableArea;
class SHorizontalBox;
class SSearchBox;
class STextComboBox;
class SVerticalBox;
class UCustomizableObject;
class UCustomizableObjectPopulationClass;
struct FGeometry;
struct FSlateBrush;

// Widget to manage a single tag 
class SSingleTagWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSingleTagWidget) {}
		SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
		SLATE_ARGUMENT(class SCustomizableObjectPopulationClassTagsTool*, TagsToolPtr)
		SLATE_ARGUMENT(int32, TagIndex)
		SLATE_ARGUMENT(TArray<FString>*, TagsList)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback for OnSelectionchanged of the tags ComboBox */
	void OnTagsComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Callback for OnClicked of the remove tag button */
	FReply OnRemoveTagButtonPressed();

private:

	// Pointer to the population class that is being edited
	UCustomizableObjectPopulationClass* PopulationClass;

	/** List with all the tags for the tags ComboBox Options */
	TArray<TSharedPtr<FString>> ComboBoxTagsOptions;

	/** ComboBox with all the possigle tags */
	TSharedPtr<STextComboBox> TagsComboBox;

	/** Index of the tag */
	int32 TagIndex;

	/** Pointer to the tags tool */
	class SCustomizableObjectPopulationClassTagsTool* TagsToolPtr;

	// List of tags of the selected parameter
	TArray<FString>* TagsList;
};


// Widget to manage the tags of a parameter
class SParameterTagWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SParameterTagWidget){}
		SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
		SLATE_ARGUMENT(class SCustomizableObjectPopulationClassTagsTool*, TagsToolPtr)
		SLATE_ARGUMENT(TArray<FString>*, TagsList)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Pointer to the population class that is being edited */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** Pointer to the tags tool */
	class SCustomizableObjectPopulationClassTagsTool* TagsToolPtr;

	// List of tags of the selected parameter
	TArray<FString>* TagsList;

	/** Row representation of the tags */
	TSharedPtr<SVerticalBox> Rows;
};


//Tool to manage the tags of the parameters and its parameter options
class SCustomizableObjectPopulationClassTagsTool : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectPopulationClassTagsTool) {}
		SLATE_ARGUMENT(UCustomizableObjectPopulationClass*, PopulationClass)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	/** Widget Generators */
	void GenerateTagsManager();
	void GenerateTagsViewer();
	void GenerateParameterManager();
	void GenerateParameterOptionsManager();
	void GenerateChildSlot();

	/** Callback for the OnClicked of the RemoveTag button */
	FReply OnRemoveTagButtonPressed(int32 Index);
	
	/** Callback for the OnClicked of the RemoveTag button */
	FReply OnAddTagButtonPressed();

	/** Callback for the OnTextChanged of the Searchbox */
	void OnSearchBoxFilterTextChanged(const FText& InText);

	/** Callback for the OnTextCommited of the Searchbox */
	void OnSearchBoxTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Callback for the OnSelectionChanged of parameters ComboBox */
	void OnParameterComboBoxSelectionChanged(TSharedPtr<FString> Selection, ESelectInfo::Type SelectInfo);

	/** Callback for the OnClicked AddTag button of a parameter */
	FReply OnParameterAddTagButtonPressed();

	/** Callback for the OnClicked AddTag button of a parameter optiion */
	FReply OnParameterOptionAddTagButtonPressed(FString ParameterOptionName);

	/** Returns the image used to render the expandable area title bar with respect to its hover/expand state. */
	const FSlateBrush* GetExpandableAreaBorderImage(const SExpandableArea& Area);

private:

	/** Pointer to the population class that is being edited */
	UCustomizableObjectPopulationClass* PopulationClass;

	/** Pointer to the Customizable Object of the Population Class */
	UCustomizableObject* SelectedCustomizableObject;

	/** Expandable area widget of the tags manager */
	TSharedPtr<SExpandableArea> TagsManagerExpandableArea;

	/** Widget that stores the Tags Manager */
	TSharedPtr<SVerticalBox> TagsManager;

	/** Widget that shows the Tags */
	TSharedPtr<SVerticalBox> TagsViewer;

	/** Widget that stores the Parameters Manager */
	TSharedPtr<SHorizontalBox> ParametersManager;

	/** Expandable area widget of the parameters options manager */
	TSharedPtr<SExpandableArea> ParameterOptionsManagerExpandableArea;

	/** Widget that stores the Parameter Options Manager */
	TSharedPtr<SVerticalBox> ParameterOptionsManager;

	/** Search box of the ComboButton */
	TSharedPtr<SSearchBox> SearchBoxWidget;

	/** Stores the input of the Searchboc widget*/
	FString SearchItem;

	/** Parameters Combobox Widget */
	TSharedPtr<STextComboBox> ParameterComboBox;

	/** Options shown in the parameters combobox */
	TArray<TSharedPtr<FString>> ParameterComboBoxOptions;

	/** Selected parameter of the parameter combobox*/
	FString LastSelectedParameter;

	/** Widget Scrollbox */
	TSharedPtr<class SScrollBox> ScrollBox;
	TSharedPtr<class SScrollBox> TagsScrollBox;

	/** if true means that the tool needs to be redrawed */
	bool bNeedsUpdate;

};	
