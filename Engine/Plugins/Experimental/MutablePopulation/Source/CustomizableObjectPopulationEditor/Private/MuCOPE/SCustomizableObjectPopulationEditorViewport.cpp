// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/SCustomizableObjectPopulationEditorViewport.h"

#include "AdvancedPreviewScene.h"
#include "Components/CapsuleComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Editor/UnrealEdTypes.h"
#include "MuCOPE/CustomizableObjectPopulationEditorViewportClient.h"
#include "MuCOPE/SCustomizableObjectPopulationEditorViewportToolBar.h"
#include "Viewports.h"

class FEditorViewportClient;
class SWidget;
struct FGeometry;


#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationEditorViewport"


void SCustomizableObjectPopulationEditorViewport::Construct(const FArguments & InArgs)
{
	SEditorViewport::Construct(SEditorViewport::FArguments());
}


void SCustomizableObjectPopulationEditorViewport::Tick(const FGeometry & AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SEditorViewport::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}


TSharedRef<FEditorViewportClient> SCustomizableObjectPopulationEditorViewport::MakeEditorViewportClient()
{
	PreviewScene = MakeShareable(new FAdvancedPreviewScene(FPreviewScene::ConstructionValues()));

	EditorViewportClient = MakeShareable(new FCustomizableObjectPopulationEditorViewportClient(PreviewScene.ToSharedRef()));
	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	EditorViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);
	EditorViewportClient->SetRealtime(true);

	return EditorViewportClient.ToSharedRef();
}


void SCustomizableObjectPopulationEditorViewport::SetPreviewComponents(const TArray<USkeletalMeshComponent*>& SkeletalMeshes, const TArray<UCapsuleComponent*>& ColliderComponents,
	 int32 Columns, int32 Separation)
{
	int32 Rows = 0;

	// Removing the components of the previous population
	for (int32 i = 0; i < PreviewSkeletalMeshes.Num(); ++i)
	{
		PreviewScene->RemoveComponent(PreviewSkeletalMeshes[i]);
		PreviewScene->RemoveComponent(PreviewCollisionComponents[i]);
	}

	PreviewSkeletalMeshes.Empty();
	PreviewCollisionComponents.Empty();
	
	if (SkeletalMeshes.Num() > 0)
	{
		// Clonning the input array
		PreviewSkeletalMeshes = ObjectPtrWrap(SkeletalMeshes);
		PreviewCollisionComponents = ColliderComponents;

		for (int32 i = 0; i < PreviewSkeletalMeshes.Num(); ++i)
		{
			if (SkeletalMeshes[i])
			{
				// Setting the position of each population instance
				FTransform Transform = FTransform::Identity;
				Transform.SetLocation(FVector(i% Columns * Separation , Rows * -Separation, 0.0f));

				// Adding the component to the Scene
				PreviewScene->AddComponent(SkeletalMeshes[i], Transform);
				PreviewScene->AddComponent(ColliderComponents[i], Transform);
				PreviewSkeletalMeshes[i]->MarkRenderStateDirty();

				if ((i + 1) % Columns == 0)
				{
					Rows++;
				}
			}
		}

		// Sending the components to the viewport client
		EditorViewportClient->SetPreviewComponent(SkeletalMeshes, ColliderComponents);
	}
}


void SCustomizableObjectPopulationEditorViewport::RefreshViewport()
{
	//reregister the preview components, so if the preview material changed it will be propagated to the render thread
	for (int32 i = 0; i < PreviewSkeletalMeshes.Num(); ++i)
	{
		PreviewSkeletalMeshes[i]->MarkRenderStateDirty();
	}

	EditorViewportClient->Invalidate();
}


void SCustomizableObjectPopulationEditorViewport::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (int32 i = 0; i < PreviewSkeletalMeshes.Num(); ++i)
	{
		Collector.AddReferencedObject(PreviewSkeletalMeshes[i]);
	}
}

void SCustomizableObjectPopulationEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	Overlay->AddSlot()
		.VAlign(VAlign_Top)
		[
			SNew(SCustomizableObjectPopulationEditorViewportToolBar,/* TabBodyPtr.Pin(),*/ SharedThis(this))
			.Cursor(EMouseCursor::Default)
		];
}

int32 SCustomizableObjectPopulationEditorViewport::GetSelectedInstance() 
{ 
	if (EditorViewportClient.IsValid())
	{
		return EditorViewportClient->GetSelectedInstance();
	}
	else
	{
		return -1;
	}
}

#undef LOCTEXT_NAMESPACE
