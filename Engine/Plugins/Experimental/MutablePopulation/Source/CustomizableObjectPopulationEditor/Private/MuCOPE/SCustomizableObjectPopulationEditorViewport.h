// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "UObject/GCObject.h"

class FAdvancedPreviewScene;
class FEditorViewportClient;
class FReferenceCollector;
class SWidget;
class UCapsuleComponent;
class USkeletalMeshComponent;
struct FGeometry;


class SCustomizableObjectPopulationEditorViewport : public SEditorViewport, public FGCObject
{
public:

	SLATE_BEGIN_ARGS(SCustomizableObjectPopulationEditorViewport) {} // the OwnerHUD var is passed to the widget so the owner can be set.
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;


	// SEditorViewport interface
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;

	void SetPreviewComponents(const TArray<USkeletalMeshComponent*>& SkeletalMeshes,
														const TArray<class UCapsuleComponent*>& ColliderComponents,
														int32 Columns,
														int32 Separation);

	void RefreshViewport();

	void AddReferencedObjects(FReferenceCollector& Collector) override;

	// Returns the index of the selected instance in the viewport
	int32 GetSelectedInstance();

	virtual FString GetReferencerName() const override
	{
		return TEXT("SCustomizableObjectPopulationEditorViewport");
	}

private:

	/** Editor viewport client */
	TSharedPtr<class FCustomizableObjectPopulationEditorViewportClient> EditorViewportClient;

	/** The preview scene that we are viewing */
	TSharedPtr< FAdvancedPreviewScene > PreviewScene;

	/** Current skeletal meshes */
	TArray<TObjectPtr<USkeletalMeshComponent>> PreviewSkeletalMeshes;

	/** Current Collider Components */
	TArray<UCapsuleComponent*> PreviewCollisionComponents;

	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
};
