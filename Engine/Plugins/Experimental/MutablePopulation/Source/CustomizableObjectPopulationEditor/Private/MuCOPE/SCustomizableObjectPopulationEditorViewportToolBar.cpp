// Copyright Epic Games, Inc. All Rights Reserved.

#include "MuCOPE/SCustomizableObjectPopulationEditorViewportToolBar.h"

#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "MuCOPE/SCustomizableObjectPopulationEditorViewport.h"
#include "SEditorViewportViewMenu.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBorder.h"

class SWidget;

#define LOCTEXT_NAMESPACE "CustomizableObjectPopulationEditorViewportToolBar"

///////////////////////////////////////////////////////////
// SCustomizableObjectEditorViewportToolBar


void SCustomizableObjectPopulationEditorViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<class SCustomizableObjectPopulationEditorViewport> InViewport)
{
	Viewport = InViewport;

	TSharedRef<SCustomizableObjectPopulationEditorViewport> ViewportRef = Viewport.Pin().ToSharedRef();

	TSharedRef<SHorizontalBox> LeftToolbar = SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 2.0f))
		[
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
		.Cursor(EMouseCursor::Default)
		.Image("EditorViewportToolBar.MenuDropdown")
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("EditorViewportToolBar.MenuDropdown")))
		.OnGetMenuContent(this, &SCustomizableObjectPopulationEditorViewportToolBar::GenerateOptionsMenu)
		]

	// View menu (lit, unlit, etc...)
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 2.0f)
		[
			SNew(SEditorViewportViewMenu, InViewport.ToSharedRef(), SharedThis(this))
		];

	static const FName DefaultForegroundName("DefaultForeground");

	FLinearColor ButtonColor1 = FLinearColor(0.1f, 0.1f, 0.1f, 1.0f);
	FLinearColor ButtonColor2 = FLinearColor(0.2f, 0.2f, 0.2f, 0.75f);
	FLinearColor TextColor1 = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
	FLinearColor TextColor2 = FLinearColor(0.8f, 0.8f, 0.8f, 0.8f);
	FSlateFontInfo Info = FAppStyle::GetFontStyle("BoldFont");
	Info.Size += 26;

	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
		//// Color and opacity is changed based on whether or not the mouse cursor is hovering over the toolbar area
		//.ColorAndOpacity(this, &SViewportToolBar::OnGetColorAndOpacity)
		.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		[
			LeftToolbar
		]
		]
		]
		]
		];

	SViewportToolBar::Construct(SViewportToolBar::FArguments());

	//CompileErrorLayout.Get()->SetVisibility(EVisibility::Hidden);
}

TSharedRef<SWidget> SCustomizableObjectPopulationEditorViewportToolBar::GenerateViewMenu() const
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ViewMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());

	return ViewMenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCustomizableObjectPopulationEditorViewportToolBar::GenerateViewportTypeMenu() const
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder CameraMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList());

	// Camera types
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Perspective);

	CameraMenuBuilder.BeginSection("LevelViewportCameraType_Ortho", LOCTEXT("CameraTypeHeader_Ortho", "Orthographic"));
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Top);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Bottom);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Left);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Right);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Front);
	CameraMenuBuilder.AddMenuEntry(FEditorViewportCommands::Get().Back);
	CameraMenuBuilder.EndSection();

	return CameraMenuBuilder.MakeWidget();
}

FSlateColor SCustomizableObjectPopulationEditorViewportToolBar::GetFontColor() const
{
	return FLinearColor::White;
}

float SCustomizableObjectPopulationEditorViewportToolBar::OnGetFOVValue() const
{
	return Viewport.Pin()->GetViewportClient()->ViewFOV;
}

void SCustomizableObjectPopulationEditorViewportToolBar::OnFOVValueChanged(float NewValue) const
{
	TSharedPtr<FEditorViewportClient> ViewportClient = Viewport.Pin()->GetViewportClient();
	ViewportClient->ViewFOV = NewValue;
	ViewportClient->Invalidate();
}

FReply SCustomizableObjectPopulationEditorViewportToolBar::OnMenuClicked()
{
	// If the menu button is clicked toggle the state of the menu anchor which will open or close the menu
	if (MenuAnchor->ShouldOpenDueToClick())
	{
		MenuAnchor->SetIsOpen(true);
		this->SetOpenMenu(MenuAnchor);
	}
	else
	{
		MenuAnchor->SetIsOpen(false);
		TSharedPtr<SMenuAnchor> NullAnchor;
		this->SetOpenMenu(MenuAnchor);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SCustomizableObjectPopulationEditorViewportToolBar::GenerateOptionsMenu() const
{
	// Get all menu extenders for this context menu from the level editor module
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	TArray<FLevelEditorModule::FLevelEditorMenuExtender> MenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportOptionsMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute(Viewport.Pin()->GetCommandList().ToSharedRef()));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	const bool bIsPerspective = Viewport.Pin()->GetViewportClient()->IsPerspective();
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder OptionsMenuBuilder(bInShouldCloseWindowAfterMenuSelection, Viewport.Pin()->GetCommandList(), MenuExtender);
	{
		OptionsMenuBuilder.AddWidget(GenerateFOVMenu(), LOCTEXT("FOVAngle", "Field of View (H)"));
	}

	return OptionsMenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SCustomizableObjectPopulationEditorViewportToolBar::GenerateFOVMenu() const
{
	const float FOVMin = 5.f;
	const float FOVMax = 170.f;

	return
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.WidthOverride(100.0f)
		[
			SNew(SSpinBox<float>)
			.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
		.MinValue(FOVMin)
		.MaxValue(FOVMax)
		.Value(this, &SCustomizableObjectPopulationEditorViewportToolBar::OnGetFOVValue)
		.OnValueChanged(this, &SCustomizableObjectPopulationEditorViewportToolBar::OnFOVValueChanged)
		]
		];
}

#undef LOCTEXT_NAMESPACE
