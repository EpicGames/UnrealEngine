// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SViewportToolBar.h"

namespace ETextCommit { enum Type : int; }

class SMenuAnchor;
class SWidget;


/**
* A level viewport toolbar widget that is placed in a viewport
*/
class SCustomizableObjectPopulationEditorViewportToolBar : public SViewportToolBar
{
public:
	SLATE_BEGIN_ARGS(SCustomizableObjectPopulationEditorViewportToolBar) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<class SCustomizableObjectPopulationEditorViewport> InViewport);

private:
	/**
	* Generates the toolbar view menu content
	*
	* @return The widget containing the view menu content
	*/
	TSharedRef<SWidget> GenerateViewMenu() const;

	/**
	* Generates the toolbar viewport type menu content
	*
	* @return The widget containing the viewport type menu content
	*/
	TSharedRef<SWidget> GenerateViewportTypeMenu() const;

	/**
	* Generate color of the text on the top
	*/
	FSlateColor GetFontColor() const;

	/** Called by the FOV slider in the perspective viewport to get the FOV value */
	float OnGetFOVValue() const;
	/** Called when the FOV slider is adjusted in the perspective viewport */
	void OnFOVValueChanged(float NewValue) const;
	/** Called when a value is entered into the FOV slider/box in the perspective viewport */
	void OnFOVValueCommitted(float NewValue, ETextCommit::Type CommitInfo) {}

	/** Callback for drop-down menu with FOV and high resolution screenshot options currently */
	FReply OnMenuClicked();

	/** Generates drop-down menu with FOV and high resolution screenshot options currently */
	TSharedRef<SWidget> GenerateOptionsMenu() const;

	/** Generates widgets for viewport camera FOV control */
	TSharedRef<SWidget> GenerateFOVMenu() const;

private:
	/** The viewport that we are in */
	TWeakPtr<class SCustomizableObjectPopulationEditorViewport> Viewport;

	// Layout to show information about instance skeletal mesh update / CO asset data
	TSharedPtr<class SButton> CompileErrorLayout;

	// Layout to show if the skeletal mesh of a functional CO is being updated
	TSharedPtr<class SButton> UpdateInfoLayout;

	SViewportToolBar* CastedViewportToolbar;

	TSharedPtr<SMenuAnchor> MenuAnchor;
};
