// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

class FCustomizableObjectPopulationClassEditorCommands : public TCommands<FCustomizableObjectPopulationClassEditorCommands>
{
public:

	FCustomizableObjectPopulationClassEditorCommands();

	TSharedPtr< FUICommandInfo > Save;
	TSharedPtr< FUICommandInfo > SaveCustomizableObject;
	TSharedPtr< FUICommandInfo > OpenCustomizableObjectEditor;
	TSharedPtr< FUICommandInfo > SaveEditorCurve;
	TSharedPtr< FUICommandInfo > TestPopulationClass;
	TSharedPtr< FUICommandInfo > GenerateInstances;
	TSharedPtr< FUICommandInfo > InspectSelectedInstance;
	TSharedPtr< FUICommandInfo > InspectSelectedSkeletalMesh;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

};
