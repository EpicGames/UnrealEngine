// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CustomizableObjectPopulationClassFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectPopulationClassFactory : public UFactory
{
	GENERATED_BODY()

	UCustomizableObjectPopulationClassFactory();

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	// End UFactory Interface

};
