// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FUICommandInfo;

class FCustomizableObjectPopulationEditorCommands : public TCommands<FCustomizableObjectPopulationEditorCommands>
{

public:
	FCustomizableObjectPopulationEditorCommands();

	TSharedPtr< FUICommandInfo > Save;
	TSharedPtr< FUICommandInfo > TestPopulation;
	TSharedPtr< FUICommandInfo > GenerateInstances;
	TSharedPtr< FUICommandInfo > InspectSelectedInstance;
	TSharedPtr< FUICommandInfo > InspectSelectedSkeletalMesh;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

};
