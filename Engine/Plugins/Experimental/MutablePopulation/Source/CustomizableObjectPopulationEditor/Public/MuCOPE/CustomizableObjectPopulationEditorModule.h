// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkit.h"

class ICustomizableObjectPopulationClassEditor;
class ICustomizableObjectPopulationEditor;
class IToolkitHost;
class UCustomizableObjectPopulation;
class UCustomizableObjectPopulationClass;

extern const FName CustomizableObjectEditorAppIdentifier;
extern const FName CustomizableObjectInstanceEditorAppIdentifier;
extern const FName CustomizableObjectPopulationEditorAppIdentifier;
extern const FName CustomizableObjectPopulationClassEditorAppIdentifier;

/**
 * Customizable object population editor module interface
 */
class ICustomizableObjectPopulationEditorModule : public IModuleInterface
{
public:

	//
	static inline ICustomizableObjectPopulationEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< ICustomizableObjectPopulationEditorModule >("CustomizableObjectPopulationEditor");
	}

	/**
	 * Creates a new customizable object population editor.
	 */
	virtual TSharedRef<ICustomizableObjectPopulationEditor> CreateCustomizableObjectPopulationEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulation* CustomizablePopulation) = 0;
	virtual TSharedRef<ICustomizableObjectPopulationClassEditor> CreateCustomizableObjectPopulationClassEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UCustomizableObjectPopulationClass* CustomizablePopulationClass) = 0;
	virtual TSharedPtr<class FExtensibilityManager> GetCustomizableObjectPopulationEditorToolBarExtensibilityManager() { return nullptr; }


};
 
