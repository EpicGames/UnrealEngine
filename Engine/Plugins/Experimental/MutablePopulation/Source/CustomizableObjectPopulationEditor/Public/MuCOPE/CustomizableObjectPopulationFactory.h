// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CustomizableObjectPopulationFactory.generated.h"

class FFeedbackContext;
class UClass;
class UObject;


UCLASS(MinimalAPI)
class UCustomizableObjectPopulationFactory : public UFactory
{
	GENERATED_BODY()

	UCustomizableObjectPopulationFactory();

	// Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	// End UFactory Interface

};
