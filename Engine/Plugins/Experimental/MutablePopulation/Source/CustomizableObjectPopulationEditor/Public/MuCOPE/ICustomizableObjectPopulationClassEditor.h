// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class UCustomizableObjectPopulationClass;

/**
 * Public interface to Static Mesh Editor
 */
class ICustomizableObjectPopulationClassEditor : public FAssetEditorToolkit
{
public:
	/** Retrieves the current custom asset. */
	virtual UCustomizableObjectPopulationClass* GetCustomAsset() = 0;

	/** Set the current custom asset. */
	virtual void SetCustomAsset(UCustomizableObjectPopulationClass* InCustomAsset) {};
};
