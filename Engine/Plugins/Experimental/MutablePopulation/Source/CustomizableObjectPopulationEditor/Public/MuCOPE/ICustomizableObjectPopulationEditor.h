// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class UCustomizableObjectPopulation;

/**
 * Public interface to Static Mesh Editor
 */
class ICustomizableObjectPopulationEditor : public FAssetEditorToolkit
{
public:
	/** Retrieves the current custom asset. */
	virtual UCustomizableObjectPopulation* GetCustomAsset() = 0;

	/** Set the current custom asset. */
	virtual void SetCustomAsset(UCustomizableObjectPopulation* InCustomAsset) {};
};
